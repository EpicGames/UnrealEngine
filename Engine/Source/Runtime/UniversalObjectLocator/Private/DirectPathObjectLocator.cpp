// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectPathObjectLocator.h"
#include "UniversalObjectLocatorInitializeParams.h"
#include "UniversalObjectLocatorInitializeResult.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "UniversalObjectLocatorStringParams.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DirectPathObjectLocator)

UE::UniversalObjectLocator::TFragmentTypeHandle<FDirectPathObjectLocator> FDirectPathObjectLocator::FragmentType;

UE::UniversalObjectLocator::FResolveResult FDirectPathObjectLocator::Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const
{
	using namespace UE::UniversalObjectLocator;

	if (UObject* Object = Path.ResolveObject())
	{
		return FResolveResultData(Object);
	}

	if (EnumHasAnyFlags(Params.Flags, ELocatorResolveFlags::Load))
	{
		UObject* Object = Path.TryLoad();

		FResolveResultData Result(Object);
		Result.Flags.bWasLoaded = Object != nullptr;
		return Result;
	}

	return FResolveResult();
}

UE::UniversalObjectLocator::FInitializeResult FDirectPathObjectLocator::Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams)
{
	using namespace UE::UniversalObjectLocator;

	Path.SetPath(InParams.Object->GetPathName());
	return FInitializeResult::Absolute();
}

void FDirectPathObjectLocator::ToString(FStringBuilderBase& OutStringBuilder) const
{
	Path.AppendString(OutStringBuilder);
}

UE::UniversalObjectLocator::FParseStringResult FDirectPathObjectLocator::TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params)
{
	Path = InString;
	return UE::UniversalObjectLocator::FParseStringResult().Success();
}

uint32 FDirectPathObjectLocator::ComputePriority(const UObject* ObjectToReference, const UObject* Context)
{
	// We can't use this at all unless explicitly told to
	return 0;
}

