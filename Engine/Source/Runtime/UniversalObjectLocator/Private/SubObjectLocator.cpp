// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubObjectLocator.h"
#include "UniversalObjectLocatorInitializeParams.h"
#include "UniversalObjectLocatorInitializeResult.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "UniversalObjectLocatorStringParams.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubObjectLocator)

UE::UniversalObjectLocator::TFragmentTypeHandle<FSubObjectLocator> FSubObjectLocator::FragmentType;

UE::UniversalObjectLocator::FResolveResult FSubObjectLocator::Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const
{
	using namespace UE::UniversalObjectLocator;

	if (UObject* NonConstContext = const_cast<UObject*>(Params.Context))
	{
		UObject* Result = nullptr;
		if (!NonConstContext->ResolveSubobject(*PathWithinContext, Result, false))
		{
			Result = FindObject<UObject>(NonConstContext, *PathWithinContext);
		}
		return FResolveResultData(Result);
	}
	return FResolveResult();
}

UE::UniversalObjectLocator::FInitializeResult FSubObjectLocator::Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams)
{
	using namespace UE::UniversalObjectLocator;

	PathWithinContext.Reset();
	if (ensureMsgf(InParams.Context && InParams.Object->IsIn(InParams.Context),
		TEXT("Unable to create a reference to %s from context %s since the object does not exist within the context"),
		*InParams.Object->GetName(),
		InParams.Context ? *InParams.Context->GetName() : TEXT("<<none>>"))
		)
	{
		InParams.Object->GetPathName(InParams.Context, PathWithinContext);

		return FInitializeResult::Relative(InParams.Context);
	}

	return FInitializeResult::Failure();
}

void FSubObjectLocator::ToString(FStringBuilderBase& OutStringBuilder) const
{
	OutStringBuilder.Append(PathWithinContext);
}

UE::UniversalObjectLocator::FParseStringResult FSubObjectLocator::TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params)
{
	PathWithinContext.Reset(InString.Len());
	PathWithinContext.Append(InString.GetData(), InString.Len());
	return UE::UniversalObjectLocator::FParseStringResult().Success();
}

uint32 FSubObjectLocator::ComputePriority(const UObject* ObjectToReference, const UObject* Context)
{
	// Can only reference objects that are relative to the context
	if (Context && ObjectToReference->IsIn(Context))
	{
		return 1000;
	}

	// We can't use this at all
	return 0;
}

