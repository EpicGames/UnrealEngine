// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniversalObjectLocator.h"
#include "DirectPathObjectLocator.h"
#include "UniversalObjectLocatorInitializeParams.h"
#include "UniversalObjectLocatorInitializeResult.h"
#include "UniversalObjectLocatorStringParams.h"
#include "UniversalObjectLocatorStringUtils.h"
#include "UniversalObjectLocatorRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UniversalObjectLocator)

#define LOCTEXT_NAMESPACE "UOL"

namespace UE::UniversalObjectLocator
{
	static constexpr FStringView MagicLeadingString = TEXTVIEW("uobj://");

	const FFragmentType* FindBestFragmentType(const UObject* Object, UObject* Context);

} // namespace UE::UniversalObjectLocator

bool operator==(const FUniversalObjectLocator& A, const FUniversalObjectLocator& B)
{
	return A.Fragments == B.Fragments;
}

bool operator!=(const FUniversalObjectLocator& A, const FUniversalObjectLocator& B)
{
	return A.Fragments != B.Fragments;
}

uint32 GetTypeHash(const FUniversalObjectLocator& Locator)
{
	return GetTypeHash(Locator.Fragments);
}

FUniversalObjectLocator::FUniversalObjectLocator()
{

}

FUniversalObjectLocator::FUniversalObjectLocator(UObject* Object, UObject* Context, UObject* StopAtContext)
{
	Reset(Object, Context, StopAtContext);
}

UE::UniversalObjectLocator::FResolveResult FUniversalObjectLocator::Resolve(const FResolveParams& Params) const
{
	using namespace UE::UniversalObjectLocator;

	if (UE::IsSavingPackage(nullptr) || IsGarbageCollecting())
	{
		return FResolveResult();
	}

	// Check for invalid combinations of flags
	check(!EnumHasAllFlags(Params.Flags, ELocatorResolveFlags::Load | ELocatorResolveFlags::Unload));
	// Cannot have WillWait without Async
	check(!EnumHasAllFlags(Params.Flags, ELocatorResolveFlags::WillWait) || EnumHasAllFlags(Params.Flags, ELocatorResolveFlags::Async));

	if (EnumHasAnyFlags(Params.Flags, ELocatorResolveFlags::Async))
	{
		return ResolveAsyncImpl(Params);
	}
	else
	{
		return ResolveSyncImpl(Params);
	}
}

UE::UniversalObjectLocator::FResolveResult FUniversalObjectLocator::ResolveSyncImpl(const FResolveParams& Params) const
{
	using namespace UE::UniversalObjectLocator;

	check(!EnumHasAnyFlags(Params.Flags, ELocatorResolveFlags::Async));

	FResolveResult EmptyResult;

	if (Fragments.Num() == 0)
	{
		return EmptyResult;
	}

	if (Fragments.Num() == 1)
	{
		return Fragments[0].Resolve(Params);
	}

	bool bLoadedIndirectly = false;
	
	FResolveResult LastResult;

	UObject* CurrentContext = Params.Context;
	const int32 Num = Fragments.Num();
	for (int32 Index = 0; Index < Num; ++Index)
	{
		const FUniversalObjectLocatorFragment& Fragment = Fragments[Index];

		const bool bLastFragment = Index == (Num-1);

		// Only unload the last one
		FResolveParams RelativeParams(CurrentContext, Params.Flags);
		if (!bLastFragment)
		{
			RelativeParams.Flags &= ~ELocatorResolveFlags::Unload;
		}

		LastResult = Fragment.Resolve(RelativeParams);

		if (EnumHasAnyFlags(RelativeParams.Flags, ELocatorResolveFlags::Unload))
		{
			return LastResult;
		}

		FResolveResultData ResultData = LastResult.SyncGet();
		if (ResultData.Object == nullptr)
		{
			// If anything fails to resolve, nothing resolves
			return EmptyResult;
		}

		CurrentContext = ResultData.Object;

		// If the last one was implicitly loaded or created, the final result should report that
		
		if (bLastFragment)
		{
			ResultData.Flags.bWasLoadedIndirectly = bLoadedIndirectly;
		}
		else
		{
			bLoadedIndirectly |= ResultData.Flags.bWasLoaded;
		}
	}

	return LastResult;
}

UE::UniversalObjectLocator::FResolveResult FUniversalObjectLocator::ResolveAsyncImpl(const FResolveParams& Params) const
{
	using namespace UE::UniversalObjectLocator;

	check(EnumHasAnyFlags(Params.Flags, ELocatorResolveFlags::Async));

	if (Fragments.Num() == 0)
	{
		return FResolveResult();
	}

	struct FState : TSharedFromThis<FState>
	{
		FState(const TArray<FUniversalObjectLocatorFragment>& InFragments, ELocatorResolveFlags InInputResolveFlags)
			: FragmentsCopy(InFragments)
			, CurrentIndex(-1)
			, InputResolveFlags(InInputResolveFlags)
		{}

		void ProcessNext(FResolveResultData LastResult)
		{
			const int32 Index = ++CurrentIndex;

			const bool bFinished       = Index == FragmentsCopy.Num();
			const bool bLastFragment   = Index == (FragmentsCopy.Num()-1);

			UObject* Context = LastResult.Object;

			// If the previous one was loaded or created, any subsequent operations are loaded indirectly
			if (!bLastFragment)
			{
				bLoadedIndirectly = LastResult.Flags.bWasLoaded;
			}

			const bool bCannotContinue = Context == nullptr && Index != 0;
			if (bFinished || bCannotContinue)
			{
				if (AsyncResult.IsSet())
				{
					LastResult.Flags.bWasLoadedIndirectly = bLoadedIndirectly;
					AsyncResult->SetValue(LastResult);
				}
				else
				{
					FinalResult = FResolveResult(LastResult);
				}
				return;
			}

			// Try and resolve this one
			FResolveParams RelativeParams(Context, InputResolveFlags);

			// Only unload the last one
			if (!bLastFragment)
			{
				RelativeParams.Flags &= ~ELocatorResolveFlags::Unload;
			}

			FResolveResult Result = FragmentsCopy[Index].Resolve(RelativeParams);

			// If we don't need to wait, call the next one immediately
			if (!Result.NeedsWait())
			{
				ProcessNext(Result.SyncGet());
				return;
			}

			// We need to wait for something to complete...
			//   If the currently held FinalResult is not async, we need to make it so
			if (!FinalResult.IsAsync())
			{
				check(!AsyncResult.IsSet());
				AsyncResult.Emplace();
				FinalResult = FResolveResult(AsyncResult->GetFuture());
			}

			// Set the continuation
			Result.AsyncGet(
				[State = AsShared()](const FResolveResultData& InValue)
				{
					State->ProcessNext(InValue);
				}
			);
		}

		TArray<FUniversalObjectLocatorFragment> FragmentsCopy;

		TOptional<
			TPromise<FResolveResultData>
		> AsyncResult;

		FResolveResult FinalResult;

		int32 CurrentIndex;
		ELocatorResolveFlags InputResolveFlags;
		bool bLoadedIndirectly = false;
	};

	TSharedPtr<FState> SharedState = MakeShared<FState>(this->Fragments, Params.Flags);
	SharedState->ProcessNext(nullptr);
	return MoveTemp(SharedState->FinalResult);
}

UObject* FUniversalObjectLocator::SyncFind(UObject* Context) const
{
	return Resolve(FResolveParams::SyncFind(Context)).SyncGet().Object;
}

UObject* FUniversalObjectLocator::SyncLoad(UObject* Context) const
{
	return Resolve(FResolveParams::SyncLoad(Context)).SyncGet().Object;
}

void FUniversalObjectLocator::SyncUnload(UObject* Context) const
{
	Resolve(FResolveParams::SyncUnload(Context)).SyncGet();
}

UE::UniversalObjectLocator::FResolveResult FUniversalObjectLocator::AsyncFind(UObject* Context) const
{
	return Resolve(FResolveParams::AsyncFind(Context));
}

UE::UniversalObjectLocator::FResolveResult FUniversalObjectLocator::AsyncLoad(UObject* Context) const
{
	return Resolve(FResolveParams::AsyncLoad(Context));
}

UE::UniversalObjectLocator::FResolveResult FUniversalObjectLocator::AsyncUnload(UObject* Context) const
{
	return Resolve(FResolveParams::AsyncUnload(Context));
}

void FUniversalObjectLocator::Reset()
{
	Fragments.Empty();
}

void FUniversalObjectLocator::Reset(UObject* InObject, UObject* Context, UObject* StopAtContext)
{
	Fragments.Reset();
	if (!InObject || !AddFragment(InObject, Context, StopAtContext))
	{
		// Failed to create the locator
		Fragments.Empty();
	}
}

void FUniversalObjectLocator::AddFragment(FUniversalObjectLocatorFragment&& InFragment)
{
	Fragments.Emplace(MoveTemp(InFragment));
}

const UE::UniversalObjectLocator::FFragmentType* FUniversalObjectLocator::GetLastFragmentType() const
{
	return Fragments.Num() != 0 ? Fragments.Last().GetFragmentType() : nullptr;
}

UE::UniversalObjectLocator::FFragmentTypeHandle FUniversalObjectLocator::GetLastFragmentTypeHandle() const
{
	return Fragments.Num() != 0 ? Fragments.Last().GetFragmentTypeHandle() : UE::UniversalObjectLocator::FFragmentTypeHandle();
}

FUniversalObjectLocatorFragment* FUniversalObjectLocator::GetLastFragment()
{
	return Fragments.Num() != 0 ? &Fragments.Last() : nullptr;
}

const FUniversalObjectLocatorFragment* FUniversalObjectLocator::GetLastFragment() const
{
	return Fragments.Num() != 0 ? &Fragments.Last() : nullptr;
}

bool FUniversalObjectLocator::ForEachFragment(TFunctionRef<bool(int32, int32, const FUniversalObjectLocatorFragment&)> InFunction) const
{
	for (int32 FragmentIndex = 0, NumFragments = Fragments.Num(); FragmentIndex < NumFragments; ++FragmentIndex)
	{
		if(!InFunction(FragmentIndex, NumFragments, Fragments[FragmentIndex]))
		{
			return false;
		}
	}

	return true;
}

UE::UniversalObjectLocator::EFragmentTypeFlags FUniversalObjectLocator::GetDefaultFlags() const
{
	using namespace UE::UniversalObjectLocator;
	EFragmentTypeFlags Flags = EFragmentTypeFlags::None;

	for (const FUniversalObjectLocatorFragment& Fragment : Fragments)
	{
		if (const FFragmentType* FragmentTypePtr = Fragment.GetFragmentType())
		{
			EnumAddFlags(Flags, FragmentTypePtr->Flags);
		}
	}
	return Flags;
}

void FUniversalObjectLocator::ToString(FStringBuilderBase& OutString) const
{
	using namespace UE::UniversalObjectLocator;

	// Universal Object Locators currently write to strings of the following form:
	// uobj://fragment-type-id=payload-string!...!fragment-type-id-n=payload-string-n

	TStringBuilder<128> PayloadScratchSpace;

	OutString += MagicLeadingString;

	int32 NumPrintedTypes = 0;
	int32 NumPrintedPayloads = 0;

	// Print the fragment types as the path
	for (int32 Index = 0; Index < Fragments.Num(); ++Index)
	{
		const FUniversalObjectLocatorFragment& Fragment = Fragments[Index];
		if (const FFragmentType* FragmentTypePtr = Fragment.GetFragmentType())
		{
			if (NumPrintedTypes != 0)
			{
				OutString += '/';
			}

			FragmentTypePtr->FragmentTypeID.AppendString(OutString);
			++NumPrintedTypes;
		}
	}

	if (NumPrintedTypes == 0)
	{
		OutString += TEXT("none");
		return;
	}

	OutString += TEXT("?");

	// Print the payloads as a query string
	for (int32 Index = 0; Index < Fragments.Num(); ++Index)
	{
		const FUniversalObjectLocatorFragment& Fragment = Fragments[Index];

		const FFragmentType* FragmentTypePtr = Fragment.GetFragmentType();
		const UStruct*       FragmentStruct  = FragmentTypePtr ? FragmentTypePtr->GetStruct() : nullptr;

		if (FragmentStruct)
		{
			if (NumPrintedPayloads != 0)
			{
				OutString += '&';
			}

			PayloadScratchSpace.Reset();
			FragmentTypePtr->ToString(Fragment.GetPayload(), PayloadScratchSpace);
			if (PayloadScratchSpace.Len() != 0)
			{
				OutString.Appendf(TEXT("payload%i="), Index);
				OutString.Append(PayloadScratchSpace.ToView());
			}

			++NumPrintedPayloads;
		}
	}
}

UE::UniversalObjectLocator::FParseStringResult FUniversalObjectLocator::TryParseString(FStringView InString, const FParseStringParams& InParams)
{
	using namespace UE::UniversalObjectLocator;

	FParseStringResult Result;

	if (!InString.StartsWith(MagicLeadingString, ESearchCase::IgnoreCase))
	{
		return Result.Failure(UE_UOL_PARSE_ERROR(InParams, LOCTEXT("Error_MissingScheme", "String does not start with uobj://")));
	}

	InString = Result.Progress(InString, MagicLeadingString.Len());
	if (InString.Len() == 0)
	{
		return Result.Failure(UE_UOL_PARSE_ERROR(InParams, LOCTEXT("Error_EmptyPath", "Path is empty.")));
	}

	if (InString.Compare(TEXTVIEW("none"), ESearchCase::IgnoreCase) == 0)
	{
		Reset();
		return Result.Success();
	}

	FUniversalObjectLocator Tmp;

	// First parse the fragment types
	bool bFinishedParsingFragments = false;
	while (InString.Len() != 0 && !bFinishedParsingFragments)
	{
		// Find the next / or ? character
		const int32 NextDelimiter = UE::String::FindFirstOfAnyChar(InString, TEXTVIEW("/?#"));

		FStringView ThisFragment = InString;
		if (NextDelimiter != INDEX_NONE)
		{
			// If the delimiter is a ? this is the last fragment, but there might be
			//   some additional payload string to parse
			bFinishedParsingFragments = (InString[NextDelimiter] == '?');

			// Trim the fragment string to the delimiter that we found
			//   and move the remaining string to after the delimiter
			ThisFragment = InString.Left(NextDelimiter);

			// Now trim the rest of the string to remove this fragment.
			//   If the delimiter is a # then we have nothing more to parse
			if (InString[NextDelimiter] == '#')
			{
				// Indicate that we finished parsing at the #, but reset the string
				//   so that we stop parsing anything else
				Result.NumCharsParsed += NextDelimiter;
				InString.Reset();
			}
			else
			{
				// Progress past the delimiter
				InString = Result.Progress(InString, NextDelimiter + 1);
			}

			// Empty string - just move on to the next one
			if (ThisFragment.Len() == 0)
			{
				continue;
			}
		}
		else
		{
			// No delimiter found, so the rest of the string is parsed as a fragment type
			Result.NumCharsParsed += InString.Len();
			InString.Reset();
		}

		FUniversalObjectLocatorFragment& NewFragment = Tmp.Fragments.Emplace_GetRef();

		FParseStringResult TypeResult = NewFragment.TryParseFragmentType(ThisFragment, InParams);
		Result.NumCharsParsed += TypeResult.NumCharsParsed;
		if (!TypeResult)
		{
			return Result.Failure(
				UE_UOL_PARSE_ERROR(
					InParams,
					FText::Format(
						LOCTEXT("Error_InvalidFragmentType", "Fragment Type specifier is invalid: {0}."),
						FText::FromStringView(ThisFragment)
					)
				)
			);
		}
	}

	// Next parse the fragment payloads
	bool bFinishedParsingPayloads = false;
	while (InString.Len() != 0 && !bFinishedParsingPayloads)
	{
		// Find the next & or # character
		const int32 NextDelimiter = UE::String::FindFirstOfAnyChar(InString, TEXTVIEW("&#"));

		FStringView ThisPayload = InString;
		if (NextDelimiter != INDEX_NONE)
		{
			// If the delimiter is a # this is the last payload, so we finish parsing
			//   after this one
			bFinishedParsingFragments = (InString[NextDelimiter] == '#');

			// Trim the fragment string to the delimiter that we found
			//   and move the remaining string to after the delimiter
			ThisPayload = InString.Left(NextDelimiter);

			// Progress past the delimiter
			InString = Result.Progress(InString, NextDelimiter + 1);

			// Empty string - just move on to the next one
			if (ThisPayload.Len() == 0)
			{
				continue;
			}
		}
		else
		{
			// No delimiter - parse the remaining string as a payload and finish parsing
			Result.NumCharsParsed += InString.Len();
			InString.Reset();
		}

		static constexpr FStringView PayloadString = TEXTVIEW("payload");

		// Try and parse a fragment index - any other query string key=value pairs will be skipped
		const int32 EqualsDelimiter = UE::String::FindFirstChar(ThisPayload, '=');
		if (EqualsDelimiter != INDEX_NONE && ThisPayload.StartsWith(PayloadString, ESearchCase::IgnoreCase) && ThisPayload.Len() > 8)
		{
			int32 NumChars = 0;
			uint32 ParsedIndex = 0;
			// Parse an integer from the position after the 'payload' string
			if (!ParseUnsignedInteger(ThisPayload.RightChop(PayloadString.Len()), ParsedIndex, &NumChars) || ParsedIndex > static_cast<uint32>(std::numeric_limits<int32>::max()))
			{
				// Invalid int
				Result.NumCharsParsed += ThisPayload.Len();
				continue;
			}
			else if (ParsedIndex >= static_cast<uint32>(Tmp.Fragments.Num()))
			{
				// Invalid index
				Result.NumCharsParsed += ThisPayload.Len();
				continue;
			}

			// Handle non-sensical payload strings like payload0123another=
			if (PayloadString.Len() + NumChars != EqualsDelimiter)
			{
				Result.NumCharsParsed += ThisPayload.Len();
				continue;
			}

			// Skip over the index and =
			ThisPayload = Result.Progress(ThisPayload, PayloadString.Len() + NumChars + 1);

			if (ThisPayload.Len() == 0)
			{
				// Leave the fragment as default if there is an empty string
				continue;
			}

			const int32 FragmentIndex = static_cast<int32>(ParsedIndex);
			if (!Tmp.Fragments.IsValidIndex(FragmentIndex))
			{
				// Invalid fragment index for this payload - skip it
				continue;
			}

			FUniversalObjectLocatorFragment& Fragment = Tmp.Fragments[FragmentIndex];

			FParseStringResult PayloadResult = Fragment.TryParseFragmentPayload(ThisPayload, InParams);
			Result.NumCharsParsed += PayloadResult.NumCharsParsed;
			if (!PayloadResult)
			{
				const FFragmentType* Type = Fragment.GetFragmentType();

				return Result.Failure(
					UE_UOL_PARSE_ERROR(InParams,
						FText::Format(
							LOCTEXT("Error_InvalidPayload", "Payload is invalid for fragment type {0}: {1}."),
							FText::FromName(Type ? Type->FragmentTypeID : NAME_None),
							FText::FromStringView(ThisPayload)
						)
					)
				);
			}
		}
	}

	*this = MoveTemp(Tmp);
	return Result.Success();
}

FUniversalObjectLocator FUniversalObjectLocator::FromString(FStringView InString, const FParseStringParams& InParams)
{
	FUniversalObjectLocator Locator;
	Locator.TryParseString(InString, InParams);
	return Locator;
}

bool FUniversalObjectLocator::AddFragment(const UObject* Object, UObject* Context, UObject* StopAtContext)
{
	using namespace UE::UniversalObjectLocator;

	const FFragmentType* FragmentType = FindBestFragmentType(Object, Context);
	if (!FragmentType)
	{
		return false;
	}

	// Initialize the payload
	FUniversalObjectLocatorFragment NewLocator(*FragmentType);
	FInitializeResult Result = FragmentType->InitializePayload(NewLocator.GetPayload(), FInitializeParams { Object, Context });

	if (Result.Type == ELocatorType::Undefined)
	{
		return false;
	}
	
	// If the initialization needs to be relative to a different context, add a fragment for NewContext as well
	if (Result.Type == ELocatorType::Relative && Result.RelativeToContext != Context)
	{
		if (ensureMsgf(Result.RelativeToContext, TEXT("Payload initialization reported a relative locator but did not specify what it is relative to.")))
		{
			if (StopAtContext != Result.RelativeToContext)
			{
				AddFragment(Result.RelativeToContext, nullptr, StopAtContext);
			}
		}
	}

	// Now add ours onto the tail
	Fragments.Emplace(MoveTemp(NewLocator));
	return true;
}

bool FUniversalObjectLocator::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	static const FName NAME_SoftObjectPath = "SoftObjectPath";

	if (Tag.Type == NAME_SoftObjectProperty)
	{
		FSoftObjectPtr OldProperty;
		Slot << OldProperty;

		Fragments.Reset(1);
		Fragments.Emplace(TUniversalObjectLocatorFragment<FDirectPathObjectLocator>(OldProperty.ToSoftObjectPath()));
		return true;
	}
	else if (Tag.GetType().IsStruct(NAME_SoftObjectPath))
	{
		FSoftObjectPath OldPath;
		Slot << OldPath;

		Fragments.Reset(1);
		Fragments.Emplace(TUniversalObjectLocatorFragment<FDirectPathObjectLocator>(MoveTemp(OldPath)));
		return true;
	}

	return false;
}

bool FUniversalObjectLocator::ExportTextItem(FString& ValueStr, const FUniversalObjectLocator& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	TStringBuilder<128> String;
	ToString(String);

	ValueStr.AppendChar('(');
	ValueStr.Append(String.ToString(), String.Len());
	ValueStr.AppendChar(')');
	return true;
}

bool FUniversalObjectLocator::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText, FArchive* InSerializingArchive)
{
	using namespace UE::UniversalObjectLocator;

	if (Buffer && *Buffer == '(')
	{
		const TCHAR* BufferEnd = FCString::Strchr(Buffer, ')');
		if (Buffer != BufferEnd && (BufferEnd - Buffer) < std::numeric_limits<int32>::max())
		{
			FStringView View(Buffer + 1, int32(BufferEnd - Buffer) - 1);
			if (TryParseString(View, FParseStringParams()))
			{
				// ImportText parsing requires that we increment the buffer ptr beyond the closing ')' on success
				Buffer = BufferEnd + 1;
				return true;
			}
		}
	}

	// Try and parse this as a soft object path
	FSoftObjectPath Path;
	if (Path.ImportTextItem(Buffer, PortFlags, Parent, ErrorText, InSerializingArchive))
	{
		UObject* Object = Path.ResolveObject();
		Reset(Object);
		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
