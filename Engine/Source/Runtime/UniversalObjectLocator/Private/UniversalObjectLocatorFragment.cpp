// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniversalObjectLocatorFragment.h"
#include "Algo/Find.h"
#include "UniversalObjectLocatorFragmentType.h"
#include "UniversalObjectLocatorStringParams.h"
#include "UniversalObjectLocatorInitializeParams.h"
#include "UniversalObjectLocatorInitializeResult.h"
#include "UniversalObjectLocatorFragmentDebugging.h"
#include "UniversalObjectLocatorRegistry.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectThreadContext.h"
#include "Containers/SparseArray.h"
#include "Logging/MessageLog.h"
#include "Misc/AsciiSet.h"
#include "Misc/UObjectToken.h"
#include "Templates/AlignmentTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UniversalObjectLocatorFragment)

#define LOCTEXT_NAMESPACE "UOL"

DECLARE_LOG_CATEGORY_EXTERN(LogUniversalObjectLocator, Log, Log);
DEFINE_LOG_CATEGORY(LogUniversalObjectLocator);

namespace UE::UniversalObjectLocator
{
	const FFragmentType* FindBestFragmentType(const UObject* Object, UObject* Context)
	{
		// Loop through all our FragmentTypes to find the most supported one
		uint32 BestFragmentTypePriority = 0;
		const FFragmentType* BestFragmentType = nullptr;

		for (const FFragmentType& FragmentType : FRegistry::Get().FragmentTypes)
		{
			const uint32 ThisFragmentTypePriority = FragmentType.ComputePriority(Object, Context);
			if (ThisFragmentTypePriority > BestFragmentTypePriority)
			{
				BestFragmentTypePriority = ThisFragmentTypePriority;
				BestFragmentType = &FragmentType;
			}
		}

		if (BestFragmentType && BestFragmentType->PayloadType != nullptr)
		{
			return BestFragmentType;
		}

		return nullptr;
	}

	FFragmentType* FFragmentTypeHandle::Resolve() const
	{
		return Handle == 0xff ? nullptr : &FRegistry::Get().FragmentTypes[Handle];
	}

	FFragmentTypeHandle MakeFragmentTypeHandle(const FFragmentType* FragmentType)
	{
		check(FragmentType);

		const uint64 FragmentTypeOffset = static_cast<uint64>(FragmentType - FRegistry::Get().FragmentTypes.GetData());
		checkf(FragmentTypeOffset < std::numeric_limits<uint8>::max(), TEXT("Maximum number of UOL FragmentTypes reached"));

		return FFragmentTypeHandle(static_cast<uint8>(FragmentTypeOffset));
	}

	/**
	 * Compute the size required for the debug header of a fragment with a certain size and alignment constraint
	 * 
	 * Since our byte array is explicitly aligned to 8 bytes, we can insert the debug header right at the start of
	 * our bytes without changing the alignment of the proceeding type.
	 * 
	 * If however our type's requested alignment is greater, we use the alignment itself and allocate the header at
	 *   the tail of that space. For instance, for a 16 byte aligned payload:
	 * 0..				8..						16..													16 + sizeof(T)
	 * [				TFragmentPayload<T>		| T Payload 											]
	 *
	 * For a 32 byte aligned payload:
	 * 0..												24..					32..													32 + sizeof(T)
	 * [												TFragmentPayload<T>		| T Payload 											]
	 * 
	 */
	uint8 ComputeDebugHeaderLog2(size_t Alignment)
	{
#if UE_UNIVERSALOBJECTLOCATOR_DEBUG
		static_assert(alignof(IFragmentPayload) == 8 && sizeof(IFragmentPayload) == 8, "Unexpected alignment/size of IFragmentPayload!");

		const uint32 HeaderCapacityLog2 = FMath::CeilLogTwo((uint32)FMath::Max(Alignment, sizeof(IFragmentPayload)));

		// This value is stored in 6 bits of a uint8
		// We should never encounter a type aligned > 2^63!
		check(HeaderCapacityLog2 <= 63);

		return static_cast<uint8>(HeaderCapacityLog2);
#else
		return 0;
#endif
	}

} // UE::UniversalObjectLocator


FUniversalObjectLocatorFragment::FUniversalObjectLocatorFragment(const UObject* InObject, UObject* Context)
	: bIsInitialized(0)
	, bIsInline(0)
	, DebugHeaderSizeLog2(0)
{
	Reset(InObject, Context);
}

FUniversalObjectLocatorFragment::FUniversalObjectLocatorFragment(const UE::UniversalObjectLocator::FFragmentType& InFragmentType)
	: FragmentType(MakeFragmentTypeHandle(&InFragmentType))
	, bIsInitialized(0)
	, bIsInline(0)
	, DebugHeaderSizeLog2(0)
{
	this->DefaultConstructPayload(InFragmentType);
}

FUniversalObjectLocatorFragment::FUniversalObjectLocatorFragment()
	: bIsInitialized(0)
	, bIsInline(0)
	, DebugHeaderSizeLog2(0)
{
	static_assert(sizeof(FUniversalObjectLocatorFragment) == FUniversalObjectLocatorFragment::SizeInMemory, "Unexpected size for FUniversalObjectLocatorFragment");
	static_assert(offsetof(FUniversalObjectLocatorFragment, Data) == 0, "FUniversalObjectLocatorFragment inline data is not aligned properly");
}

FUniversalObjectLocatorFragment::~FUniversalObjectLocatorFragment()
{
	if (bIsInitialized && !IsEngineExitRequested())
	{
		DestroyPayload();
	}
}

FUniversalObjectLocatorFragment::FUniversalObjectLocatorFragment(const FUniversalObjectLocatorFragment& RHS)
	: FragmentType(RHS.FragmentType)
	, bIsInitialized(0)
	, bIsInline(0)
	, DebugHeaderSizeLog2(0)
{
	using namespace UE::UniversalObjectLocator;

	if (RHS.bIsInitialized)
	{
		const FFragmentType* ResolvedFragmentType = GetFragmentType();
		check(ResolvedFragmentType);

		this->DefaultConstructPayload(*ResolvedFragmentType);

		ResolvedFragmentType->PayloadType->CopyScriptStruct(this->GetPayload(), RHS.GetPayload());
	}
	else
	{
		this->bIsInitialized = false;
	}
}

FUniversalObjectLocatorFragment& FUniversalObjectLocatorFragment::operator=(const FUniversalObjectLocatorFragment& RHS)
{
	using namespace UE::UniversalObjectLocator;

	this->DestroyPayload();

	if (RHS.bIsInitialized)
	{
		const FFragmentType* ResolvedFragmentType = GetFragmentType();
		check(ResolvedFragmentType);

		// Assign the FragmentType and copy the payload
		this->FragmentType = RHS.FragmentType;
		this->DefaultConstructPayload(*ResolvedFragmentType);
		ResolvedFragmentType->PayloadType->CopyScriptStruct(this->GetPayload(), RHS.GetPayload());
	}
	else
	{
		this->bIsInitialized = false;
		this->FragmentType   = FFragmentTypeHandle();
	}

	return *this;
}

FUniversalObjectLocatorFragment::FUniversalObjectLocatorFragment(FUniversalObjectLocatorFragment&& RHS)
	: FragmentType(RHS.FragmentType)
	, bIsInitialized(RHS.bIsInitialized)
	, bIsInline(RHS.bIsInline)
	, DebugHeaderSizeLog2(RHS.DebugHeaderSizeLog2)
{
	using namespace UE::UniversalObjectLocator;

	FMemory::Memcpy(this->Data, RHS.Data, sizeof(Data));

	RHS.bIsInitialized      = false;
	RHS.bIsInline           = false;
	RHS.DebugHeaderSizeLog2 = 0;
	RHS.FragmentType        = FFragmentTypeHandle();
}

FUniversalObjectLocatorFragment& FUniversalObjectLocatorFragment::operator=(FUniversalObjectLocatorFragment&& RHS)
{
	using namespace UE::UniversalObjectLocator;

	this->DestroyPayload();

	this->bIsInitialized      = RHS.bIsInitialized;
	this->bIsInline           = RHS.bIsInline;
	this->DebugHeaderSizeLog2 = RHS.DebugHeaderSizeLog2;
	this->FragmentType        = RHS.FragmentType;

	FMemory::Memcpy(this->Data, RHS.Data, sizeof(Data));

	RHS.bIsInitialized      = false;
	RHS.bIsInline           = false;
	RHS.DebugHeaderSizeLog2 = 0;
	RHS.FragmentType        = FFragmentTypeHandle();

	return *this;
}

bool operator==(const FUniversalObjectLocatorFragment& A, const FUniversalObjectLocatorFragment& B)
{
	using namespace UE::UniversalObjectLocator;

	if (A.bIsInitialized != B.bIsInitialized)
	{
		return false;
	}
	else if (!A.bIsInitialized)
	{
		// 2 uninitialized references are the same
		return true;
	}
	else if (A.FragmentType != B.FragmentType)
	{
		// Different fragment types
		return false;
	}
	else
	{
		const UScriptStruct* FragmentStruct = A.GetFragmentStruct();
		check(FragmentStruct);

		// Same fragment types - compare payloads
		const void* PayloadA = A.GetPayload();
		const void* PayloadB = B.GetPayload();

		return FragmentStruct->CompareScriptStruct(PayloadA, PayloadB, 0);
	}
}

bool operator!=(const FUniversalObjectLocatorFragment& A, const FUniversalObjectLocatorFragment& B)
{
	return !(A == B);
}

uint32 GetTypeHash(const FUniversalObjectLocatorFragment& Fragment)
{
	using namespace UE::UniversalObjectLocator;

	if (!Fragment.bIsInitialized)
	{
		return 0;
	}

	uint32 Hash = GetTypeHash(Fragment.FragmentType);

	if (const FFragmentType* FragmentTypePtr = Fragment.GetFragmentType())
	{
		UScriptStruct* Struct = FragmentTypePtr->GetStruct();
		if (Struct)
		{
			const uint32 PayloadHash = Struct->GetStructTypeHash(Fragment.GetPayload());
			Hash = HashCombineFast(Hash, PayloadHash);
		}
	}

	return Hash;
}

#if DO_CHECK
void FUniversalObjectLocatorFragment::CheckPayloadType(UScriptStruct* TypeToCompare) const
{
	using namespace UE::UniversalObjectLocator;

	const FFragmentType* FragmentTypePtr = GetFragmentType();
	checkf(FragmentTypePtr == nullptr || FragmentTypePtr->PayloadType == TypeToCompare,
		TEXT("Type mismatch when accessing payload data! Attempting to access a stored %s payload as %s."),
		FragmentTypePtr->PayloadType ? *FragmentTypePtr->PayloadType->GetName() : TEXT("<expired>"),
		TypeToCompare          ? *TypeToCompare->GetName()          : TEXT("<nullptr>"));
}
#endif

void FUniversalObjectLocatorFragment::ToString(FStringBuilderBase& OutString) const
{
	using namespace UE::UniversalObjectLocator;

	const FFragmentType* FragmentTypePtr = FragmentType.Resolve();
	if (FragmentTypePtr && FragmentTypePtr->PayloadType)
	{
		FragmentTypePtr->FragmentTypeID.AppendString(OutString);

		TStringBuilder<128> PayloadString;
		FragmentTypePtr->ToString(GetPayload(), PayloadString);
		if (PayloadString.Len() != 0)
		{
			OutString += '=';
			OutString.Append(PayloadString.ToView());
		}
	}
}

UE::UniversalObjectLocator::FParseStringResult FUniversalObjectLocatorFragment::TryParseString(FStringView InString, const FParseStringParams& InParams)
{
	using namespace UE::UniversalObjectLocator;

	if (InString.Len() == 0)
	{
		Reset();
		return FParseStringResult().Success();
	}

	// Check for a literal "none" text
	static constexpr FStringView NoneString = TEXTVIEW("none");
	if (InString.Compare(NoneString, ESearchCase::IgnoreCase) == 0)
	{
		Reset();
		return FParseStringResult().Success(NoneString.Len());
	}

	FStringView FragmentTypeString = InString;
	FStringView FragmentPayloadString;

	const int32 Delimiter = UE::String::FindFirstChar(InString, '=');
	if (Delimiter != INDEX_NONE)
	{
		// We have a payload
		FragmentTypeString    = InString.Left(Delimiter);
		FragmentPayloadString = InString.RightChop(Delimiter + 1);

		if (FragmentTypeString.Len() == 0)
		{
			return FParseStringResult().Failure(UE_UOL_PARSE_ERROR(InParams, LOCTEXT("Error_UnexpectedEquals", "Unexpected '='' when expecting a fragment type.")));
		}
	}

	FParseStringResult TypeResult = TryParseFragmentType(FragmentTypeString, InParams);
	if (!TypeResult)
	{
		return TypeResult;
	}

	FParseStringResult PayloadResult = TryParseFragmentPayload(FragmentPayloadString, InParams);

	// Add the type chars and = to the total num parsed
	PayloadResult.NumCharsParsed += TypeResult.NumCharsParsed + 1;
	return PayloadResult;
}

UE::UniversalObjectLocator::FParseStringResult FUniversalObjectLocatorFragment::TryParseFragmentType(FStringView InString, const FParseStringParams& InParams)
{
	using namespace UE::UniversalObjectLocator;

	if (InString.Len() == 0)
	{
		return FParseStringResult().Failure(UE_UOL_PARSE_ERROR(InParams, LOCTEXT("Error_EmptyFragmentType", "Fragment type specifier is empty.")));
	}

	// Check for a literal "none" text
	static constexpr FStringView NoneString = TEXTVIEW("none");
	if (InString.Compare(NoneString, ESearchCase::IgnoreCase) == 0)
	{
		Reset();
		return FParseStringResult().Success(NoneString.Len());
	}

	// Try and find the FragmentType as a name
	FName FragmentTypeID(InString.Len(), InString.GetData(), FNAME_Find);
	if (FragmentTypeID != NAME_None)
	{
		// Find the FragmentType
		const FFragmentType* SerializedFragmentType = FRegistry::Get().FindFragmentType(FragmentTypeID);
		if (SerializedFragmentType != nullptr && SerializedFragmentType->PayloadType != nullptr)
		{
			this->DestroyPayload();
			this->FragmentType = MakeFragmentTypeHandle(SerializedFragmentType);
			this->DefaultConstructPayload(*SerializedFragmentType);

			return FParseStringResult().Success(InString.Len());
		}
	}

	// Not a valid fragment type string
	return FParseStringResult().Failure(
		UE_UOL_PARSE_ERROR(InParams, 
			FText::Format(
				LOCTEXT("Error_UnknownFragmentType", "Unknown fragment type specifier {0}."),
				FText::FromStringView(InString)
			)
		)
	);
}

UE::UniversalObjectLocator::FParseStringResult FUniversalObjectLocatorFragment::TryParseFragmentPayload(FStringView InString, const FParseStringParams& InParams)
{
	using namespace UE::UniversalObjectLocator;

	if (!bIsInitialized)
	{
		return FParseStringResult().Failure(UE_UOL_PARSE_ERROR(InParams, LOCTEXT("Error_Uninitialized", "Unable to parse a payload for an uninitialized fragment.")));
	}

	const FFragmentType* FragmentTypePtr = GetFragmentType();
	const UScriptStruct* FragmentStruct  = FragmentTypePtr ? FragmentTypePtr->GetStruct() : nullptr;

	if (!FragmentStruct)
	{
		return FParseStringResult().Failure(UE_UOL_PARSE_ERROR(InParams, LOCTEXT("Error_Expired", "Unable to parse a payload for a fragment whose type has expired.")));
	}

	void* Payload = GetPayload();
	if (InString.Len() == 0)
	{
		// Empty payload string means a default payload
		FragmentStruct->ClearScriptStruct(Payload);
		return FParseStringResult().Success();
	}

	return FragmentTypePtr->TryParseString(Payload, InString, InParams);
}

const UE::UniversalObjectLocator::FFragmentType* FUniversalObjectLocatorFragment::GetFragmentType() const
{
	return FragmentType.Resolve();
}

UScriptStruct* FUniversalObjectLocatorFragment::GetFragmentStruct() const
{
	using namespace UE::UniversalObjectLocator;
	if (const FFragmentType* Type = GetFragmentType())
	{
		return Type->GetStruct();
	}
	return nullptr;
}

UE::UniversalObjectLocator::FFragmentTypeHandle FUniversalObjectLocatorFragment::GetFragmentTypeHandle() const
{
	using namespace UE::UniversalObjectLocator;

	const FFragmentType* FragmentTypePtr = GetFragmentType();
	if (FragmentTypePtr)
	{
		return MakeFragmentTypeHandle(FragmentTypePtr);
	}
	return FFragmentTypeHandle();
}

void FUniversalObjectLocatorFragment::DestroyPayload()
{
	using namespace UE::UniversalObjectLocator;

	if (!bIsInitialized)
	{
		return;
	}

	uint8* Payload = (uint8*)GetPayload();

	const UScriptStruct* FragmentStruct = GetFragmentStruct();
	if (ensureMsgf(FragmentStruct, TEXT("FUniversalObjectLocatorFragment has outlived its FragmentType's payload type struct! This could leak memory if the type allocated it.")))
	{
		FragmentStruct->DestroyStruct(Payload);
	}

	if (!bIsInline)
	{
		Payload -= GetDebugHeaderOffset();
		FMemory::Free(Payload);
	}

	bIsInitialized = false;
}

void* FUniversalObjectLocatorFragment::GetPayload()
{
	check(bIsInitialized);

	uint8* Payload = bIsInline ? Data : *((uint8**)Data);
	return Payload + GetDebugHeaderOffset();
}

const void* FUniversalObjectLocatorFragment::GetPayload() const
{
	check(bIsInitialized);

	const uint8* Payload = bIsInline ? Data : *((const uint8* const *)Data);
	return Payload + GetDebugHeaderOffset();
}

FUniversalObjectLocatorFragment::FAllocatedPayload FUniversalObjectLocatorFragment::AllocatePayload(size_t Size, size_t Alignment)
{
	using namespace UE::UniversalObjectLocator;

	check(!bIsInitialized);

	bIsInitialized = true;

#if UE_UNIVERSALOBJECTLOCATOR_DEBUG
	DebugHeaderSizeLog2 = ComputeDebugHeaderLog2(Alignment);
	Alignment = FMath::Max(Alignment, alignof(IFragmentPayload));
	Size += GetDebugHeaderOffset();
#else
	DebugHeaderSizeLog2 = 0;
#endif

	uint8* Payload = nullptr;
	if (Size <= sizeof(FUniversalObjectLocatorFragment::Data) && Alignment <= alignof(FUniversalObjectLocatorFragment))
	{
		// We can placement new this into the payload data
		bIsInline = true;
		Payload = Data;
	}
	else
	{
		// We have to allocate this struct on the heap
		Payload = (uint8*)FMemory::Malloc(Size, Alignment);
		*reinterpret_cast<void**>(Data) = Payload;
		bIsInline = false;
	}

	return FAllocatedPayload{
#if UE_UNIVERSALOBJECTLOCATOR_DEBUG
		Payload + GetDebugHeaderOffset() - sizeof(IFragmentPayload),
#endif
		Payload + GetDebugHeaderOffset()
	};
}

void FUniversalObjectLocatorFragment::DefaultConstructPayload(const UE::UniversalObjectLocator::FFragmentType& InFragmentType)
{
	using namespace UE::UniversalObjectLocator;

	const UScriptStruct* PayloadType = InFragmentType.PayloadType.Get();

	FAllocatedPayload Allocation = AllocatePayload((size_t)PayloadType->GetStructureSize(), (size_t)PayloadType->GetMinAlignment());

#if UE_UNIVERSALOBJECTLOCATOR_DEBUG
	InFragmentType.StaticBindings.FragmentDebugInitializer(Allocation.DebugVFTablePtr);
#endif

	PayloadType->InitializeStruct(Allocation.Payload);
}

void FUniversalObjectLocatorFragment::Reset()
{
	using namespace UE::UniversalObjectLocator;

	DestroyPayload();
	FragmentType = FFragmentTypeHandle();
}

void FUniversalObjectLocatorFragment::Reset(const UObject* InObject, UObject* Context)
{
	using namespace UE::UniversalObjectLocator;

	Reset();

	if (const FFragmentType* BestFragmentType = FindBestFragmentType(InObject, Context))
	{
		FragmentType = MakeFragmentTypeHandle(BestFragmentType);
		DefaultConstructPayload(*BestFragmentType);

		BestFragmentType->InitializePayload(GetPayload(), FInitializeParams{ InObject, Context });
	}
}

void FUniversalObjectLocatorFragment::Reset(const UObject* InObject, UObject* Context, TFunctionRef<bool(UE::UniversalObjectLocator::FFragmentTypeHandle)> CanUseFragmentType)
{
	using namespace UE::UniversalObjectLocator;

	Reset();

	// Loop through all our FragmentTypes to find the most supported one
	uint32 BestFragmentTypePriority = 0;
	const FFragmentType* BestFragmentType = nullptr;

	TArray<FFragmentType>& FragmentTypes = FRegistry::Get().FragmentTypes;
	if (!ensure(FragmentTypes.Num() < 255))
	{
		return;
	}

	const uint8 Num = static_cast<uint8>(FragmentTypes.Num());
	for (uint8 Index = 0; Index < Num; ++Index)
	{
		if (!CanUseFragmentType(FFragmentTypeHandle(Index)))
		{
			continue;
		}

		const FFragmentType& ThisFragmentType = FragmentTypes[Index];
		const uint32 ThisFragmentTypePriority = ThisFragmentType.ComputePriority(InObject, Context);
		if (ThisFragmentTypePriority > BestFragmentTypePriority)
		{
			BestFragmentTypePriority = ThisFragmentTypePriority;
			BestFragmentType = &ThisFragmentType;
		}
	}

	if (BestFragmentType && BestFragmentType->PayloadType != nullptr)
	{
		FragmentType = MakeFragmentTypeHandle(BestFragmentType);
		DefaultConstructPayload(*BestFragmentType);

		BestFragmentType->InitializePayload(GetPayload(), FInitializeParams{ InObject, Context });
	}
}

UE::UniversalObjectLocator::FResolveResult FUniversalObjectLocatorFragment::Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const
{
	using namespace UE::UniversalObjectLocator;

	// Find our FragmentType
	const FFragmentType* FragmentTypePtr = FragmentType.Resolve();
	if (FragmentTypePtr && bIsInitialized)
	{
		return FragmentTypePtr->ResolvePayload(GetPayload(), Params);
	}

	return FResolveResult();
}

bool FUniversalObjectLocatorFragment::Serialize(FArchive& Ar)
{
	using namespace UE::UniversalObjectLocator;

	if (Ar.IsLoading())
	{
		FName FragmentTypeID;
		Ar << FragmentTypeID;

		if (FragmentTypeID == NAME_None)
		{
			Reset();
		}
		else
		{
			// Find the FragmentType
			const FFragmentType* SerializedFragmentType = FRegistry::Get().FindFragmentType(FragmentTypeID);
			if (!SerializedFragmentType || !SerializedFragmentType->PayloadType)
			{
				Reset();

				// Big error - what do we do?
				FMessageLog Log("UOL");
				TSharedRef<FTokenizedMessage> Message = Log.Error(FText::Format(LOCTEXT("DataLossWarning", "WARNING: POTENTIAL DATA LOSS! Universal Object Reference FragmentType {0}! This reference will be lost if re-saved."), FText::FromString(FragmentTypeID.ToString())));

				FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
				if (SerializeContext && SerializeContext->SerializedObject)
				{
					Message->AddToken(FUObjectToken::Create(SerializeContext->SerializedObject));
				}

				Log.Open(EMessageSeverity::Error);

				// Deserialize an empty payload so we don't corrupt the serialization data
				FUniversalObjectLocatorEmptyPayload Empty;
				FUniversalObjectLocatorEmptyPayload::StaticStruct()->SerializeItem(Ar, &Empty, nullptr);
				return true;
			}

			FragmentType = MakeFragmentTypeHandle(SerializedFragmentType);

			UScriptStruct* Struct = SerializedFragmentType->GetStruct();
			Ar.Preload(Struct);

			DefaultConstructPayload(*SerializedFragmentType);
			Struct->SerializeItem(Ar, GetPayload(), nullptr);
		}
	}
	else if (Ar.IsSaving() || Ar.IsTransacting())
	{
		const FFragmentType* FragmentTypePtr = FragmentType.Resolve();

		if (FragmentTypePtr == nullptr)
		{
			FName None;

			// FragmentType ID
			Ar << None;
		}
		else
		{
			FName FragmentTypeID = FragmentTypePtr->FragmentTypeID;

			// FragmentType ID
			Ar << FragmentTypeID;

			// FragmentType payload
			FragmentTypePtr->GetStruct()->SerializeItem(Ar, GetPayload(), nullptr);
		}
	}
	else if (Ar.IsModifyingWeakAndStrongReferences())
	{
		UScriptStruct* FragmentStruct = GetFragmentStruct();
		if (FragmentStruct && bIsInitialized)
		{
			FragmentStruct->SerializeItem(Ar, GetPayload(), nullptr);
		}
	}

	return true;
}

void FUniversalObjectLocatorFragment::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	using namespace UE::UniversalObjectLocator;

	if (FFragmentType* FragmentTypePtr = FragmentType.Resolve())
	{
		Collector.AddReferencedObject(FragmentTypePtr->PayloadType);
		if (bIsInitialized && FragmentTypePtr->PayloadType)
		{
			Collector.AddReferencedObjects(FragmentTypePtr->PayloadType, GetPayload());
		}
	}
}

bool FUniversalObjectLocatorFragment::ExportTextItem(FString& ValueStr, const FUniversalObjectLocatorFragment& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	TStringBuilder<32> PayloadString;
	ToString(PayloadString);

	ValueStr.AppendChar('(');
	ValueStr.Append(PayloadString.ToString(), PayloadString.Len());
	ValueStr.AppendChar(')');

	return true;
}

bool FUniversalObjectLocatorFragment::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText, FArchive* InSerializingArchive)
{
	using namespace UE::UniversalObjectLocator;

	if (Buffer && *Buffer == '(')
	{
		const TCHAR* BufferEnd = FCString::Strchr(Buffer, ')');
		if (Buffer != BufferEnd && (BufferEnd - Buffer) < std::numeric_limits<int32>::max())
		{
			FStringView View(Buffer+1, int32(BufferEnd-Buffer)-1);
			if (TryParseString(View, FParseStringParams()))
			{
				return true;
			}
		}
	}

	return false;
}

bool FUniversalObjectLocatorFragment::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	return false;
}

void FUniversalObjectLocatorFragment::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{

}

#undef LOCTEXT_NAMESPACE
