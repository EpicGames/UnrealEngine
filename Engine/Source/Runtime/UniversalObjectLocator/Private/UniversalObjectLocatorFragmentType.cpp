// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniversalObjectLocatorFragmentType.h"
#include "UniversalObjectLocatorFragment.h"
#include "UniversalObjectLocatorStringParams.h"
#include "UniversalObjectLocatorInitializeParams.h"
#include "UniversalObjectLocatorInitializeResult.h"

#include "GenericPlatform/GenericPlatformHttp.h"

namespace UE::UniversalObjectLocator
{

FResolveResult FFragmentType::ResolvePayload(const void* Payload, const FResolveParams& Params) const
{
	return (*InstanceBindings.Resolve)(Payload, Params);
}
FInitializeResult FFragmentType::InitializePayload(void* Payload, const FInitializeParams& InParams) const
{
	return (*InstanceBindings.Initialize)(Payload, InParams);
}
void FFragmentType::ToString(const void* Payload, FStringBuilderBase& OutStringBuilder) const
{
	// Some strings are currently emitting null terminators due to a bug with UTF8
	//   Strip these off to ensure we don't fail the check below
	while (OutStringBuilder.Len() != 0 && OutStringBuilder.LastChar() == '\0')
	{
		OutStringBuilder.RemoveSuffix(1);
	}

	(*InstanceBindings.ToString)(Payload, OutStringBuilder);

	const FStringView StringToEncode = OutStringBuilder.ToString();
	FString EncodedString = FGenericPlatformHttp::UrlEncode(StringToEncode);
	OutStringBuilder = EncodedString;

	const int32 StartPos = OutStringBuilder.Len();

#if DO_CHECK

	static constexpr FAsciiSet InvalidChars = ~FUniversalObjectLocatorFragment::ValidFragmentPayloadCharacters;

	auto MakeErrorString = [](FStringView StringView){
		
		FString String;
		for (TCHAR Char : StringView)
		{
			if (InvalidChars.Test(Char))
			{
				String.AppendChar('^');
			}
			else
			{
				String.AppendChar(' ');
			}
		}
		return String;
	};

	FStringView StringRepresentation = FStringView(OutStringBuilder).RightChop(StartPos);
	ensureMsgf(!FAsciiSet::HasAny(StringRepresentation, InvalidChars),
		TEXT("F%s::ToString resulted in an invalid character usage:\n\t%s\n\t%s"),
		*PayloadType->GetName(),
		*FString(StringRepresentation.Len(), StringRepresentation.GetData()),
		*MakeErrorString(StringRepresentation));
#endif
}
FParseStringResult FFragmentType::TryParseString(void* Payload, FStringView InString, const FParseStringParams& Params) const
{
	FString DecodedString = FGenericPlatformHttp::UrlDecode(InString);
	return (*InstanceBindings.TryParseString)(Payload, DecodedString, Params);
}

uint32 FFragmentType::ComputePriority(const UObject* Object, const UObject* Context) const
{
	return (*StaticBindings.Priority)(Object, Context);
}

} // namespace UE::UniversalObjectLocator