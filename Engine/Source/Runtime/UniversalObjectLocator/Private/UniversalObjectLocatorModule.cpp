// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubObjectLocator.h"
#include "Modules/ModuleManager.h"
#include "IUniversalObjectLocatorModule.h"
#include "UniversalObjectLocatorFragment.h"
#include "UniversalObjectLocatorRegistry.h"
#include "UniversalObjectLocatorParameterTypeHandle.h"
#include "Modules/VisualizerDebuggingState.h"
#include "Misc/DelayedAutoRegister.h"

#include "DirectPathObjectLocator.h"

namespace UE::UniversalObjectLocator
{

class FUniversalObjectLocatorModule
	: public IUniversalObjectLocatorModule
{
public:
	
	/** Debug visualizer ID. String for lookup inside natvcis should be "49ceb527db044325a786a9b4470158fc" */
	FGuid DebugVisualizerID = FGuid(0x49ceb527, 0xdb044325, 0xa786a9b4, 0x470158fc);

	void StartupModule() override
	{
		// Register fragment types as soon as the object system is ready
		FDelayedAutoRegisterHelper(EDelayedRegisterRunPhase::ObjectSystemReady,
			[this]
			{
				{
					FFragmentTypeParameters FragmentTypeParams("subobj", NSLOCTEXT("SubObjectLocator", "Object", "Object"));
					FragmentTypeParams.PrimaryEditorType = "SubObject";
					FSubObjectLocator::FragmentType = this->RegisterFragmentType<FSubObjectLocator>(FragmentTypeParams);
				}

				{
					FFragmentTypeParameters FragmentTypeParams("uobj", NSLOCTEXT("DirectPathObjectLocator", "Object", "Object"));
					FDirectPathObjectLocator::FragmentType = this->RegisterFragmentType<FDirectPathObjectLocator>(FragmentTypeParams);
				}
			}
		);
	}

	void ShutdownModule() override
	{

	}

	FFragmentTypeHandle RegisterFragmentTypeImpl(const FFragmentType& FragmentType) override
	{
		using namespace UE::Core;

		FRegistry& Registry = FRegistry::Get();

		const int32 Index = Registry.FragmentTypes.Num();
		Registry.FragmentTypes.Add(FragmentType);
		checkf(Index < static_cast<int32>(std::numeric_limits<uint8>::max()), TEXT("Maximum number of UOL FragmentTypes reached"));

		// Re-assign the debugging ptr in case it changed due to reallocation
		EVisualizerDebuggingStateResult Result = FVisualizerDebuggingState::Assign(DebugVisualizerID, Registry.FragmentTypes.GetData());

		return FFragmentTypeHandle(static_cast<uint8>(Index));
	}

	void UnregisterFragmentTypeImpl(FFragmentTypeHandle FragmentType) override
	{
		FRegistry::Get().FragmentTypes[FragmentType.GetIndex()] = FFragmentType{};
	}

	FParameterTypeHandle RegisterParameterTypeImpl(UScriptStruct* Struct)
	{
		FRegistry& Registry = FRegistry::Get();

		const int32 Index = Registry.ParameterTypes.Num();
		Registry.ParameterTypes.Add(Struct);

		checkf(Index < FResolveParameterBuffer::MaxNumParameters, TEXT("Maximum number of UOL ParameterTypes reached"));

		return FParameterTypeHandle(static_cast<uint8>(Index));
	}

	void UnregisterParameterTypeImpl(FParameterTypeHandle ParameterType)
	{
		FRegistry& Registry = FRegistry::Get();
		check(ParameterType.IsValid());
		Registry.ParameterTypes[ParameterType.GetIndex()] = nullptr;
	}
};

} // namespace UE::UniversalObjectLocator

IMPLEMENT_MODULE(UE::UniversalObjectLocator::FUniversalObjectLocatorModule, UniversalObjectLocator);

