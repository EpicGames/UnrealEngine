// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniversalObjectLocatorParameterTypeHandle.h"
#include "UniversalObjectLocatorRegistry.h"

namespace UE::UniversalObjectLocator
{


UScriptStruct* FParameterTypeHandle::Resolve(UScriptStruct* Expected) const
{
	if (Handle == 0xff)
	{
		return nullptr;
	}

	UScriptStruct* Result = FRegistry::Get().ParameterTypes[Handle];
	check(!Expected || Result == Expected);
	return Result;
}


} // namespace UE::UniversalObjectLocator