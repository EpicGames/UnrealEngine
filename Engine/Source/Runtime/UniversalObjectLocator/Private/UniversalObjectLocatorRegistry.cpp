// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniversalObjectLocatorRegistry.h"
#include "UniversalObjectLocatorFwd.h"
#include "UniversalObjectLocatorFragmentType.h"
#include "Algo/Find.h"

namespace UE::UniversalObjectLocator
{

FRegistry& FRegistry::Get()
{
	static FRegistry Registry;
	return Registry;
}

const FFragmentType* FRegistry::FindFragmentType(FName ID) const
{
	return Algo::FindBy(FragmentTypes, ID, &FFragmentType::FragmentTypeID);
}

void FRegistry::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(ParameterTypes);
	
	for (FFragmentType& FragmentType : FragmentTypes)
	{
		Collector.AddReferencedObject(FragmentType.PayloadType);
	}
}

FString FRegistry::GetReferencerName() const
{
	return TEXT("UE::UniversalObjectLocator::FRegistry");
}


} // namespace UE::UniversalObjectLocator