// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "UObject/GCObject.h"
#include "UniversalObjectLocatorFwd.h"
#include "UniversalObjectLocatorFragmentType.h"


namespace UE::UniversalObjectLocator
{

struct FRegistry
	: FGCObject
{
	static FRegistry& Get();

	const FFragmentType* FindFragmentType(FName ID) const;

	void AddReferencedObjects(FReferenceCollector& Collector) override;
	FString GetReferencerName() const override;

	TArray<TObjectPtr<UScriptStruct>> ParameterTypes;
	TArray<FFragmentType> FragmentTypes;
};

} // namespace UE::UniversalObjectLocator