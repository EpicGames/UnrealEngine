// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniversalObjectLocatorResolveParams.h"
#include "UniversalObjectLocatorFragmentType.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UniversalObjectLocatorResolveParams)

namespace UE::UniversalObjectLocator
{

FResolveParameterBuffer::FResolveParameterBuffer()
	: Memory(nullptr)
	, AllParameters(0u)
	, Capacity(0u)
	, Num(0u)
{
}

FResolveParameterBuffer::~FResolveParameterBuffer()
{
	Destroy();
}

void FResolveParameterBuffer::Destroy()
{
	check((Memory == nullptr) == (Num == 0));

	if (Memory)
	{
		uint8 Index = 0;
		uint32 RemainingParameters = AllParameters;
		while (RemainingParameters != 0)
		{
			// Get the bit offset of the next reminaing bit
			const uint8 BitIndex = static_cast<uint8>(FMath::CountTrailingZeros(RemainingParameters));
			UScriptStruct* Type = FParameterTypeHandle(BitIndex).Resolve();

			void* Ptr = GetHeader(Index).Resolve(Memory);
			++Index;

			if (ensure(Type))
			{
				Type->DestroyStruct(Ptr);
			}

			RemainingParameters = RemainingParameters & ~(1u << BitIndex);
		}

		if (bCanFreeMemory)
		{
			FMemory::Free(Memory);
		}
	}
}

} // namespace UE::UniversalObjectLocator
