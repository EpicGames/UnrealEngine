// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniversalObjectLocatorStringParams.h"
#include "Containers/StringView.h"

namespace UE::UniversalObjectLocator
{

FStringView FParseStringResult::Progress(FStringView CurrentString, int32 NumToProgress)
{
	checkf(NumToProgress <= CurrentString.Len(), TEXT("Attempting to progress past the end of the current string!"));

	NumCharsParsed += NumToProgress;
	return CurrentString.RightChop(NumToProgress);
}

} // namespace UE::UniversalObjectLocator