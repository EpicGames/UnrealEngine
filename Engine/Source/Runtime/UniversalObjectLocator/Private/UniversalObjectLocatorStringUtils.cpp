// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniversalObjectLocatorStringUtils.h"
#include "Containers/StringView.h"
#include "Misc/Char.h"

namespace UE::UniversalObjectLocator
{

bool ParseUnsignedInteger(FStringView InString, uint32& OutResult, int32* OutNumCharsParsed)
{
	if (InString.Len() == 0)
	{
		return false;
	}

	uint32 Parsed = 0;
	uint32 Factor = 1;
	int32  Index  = 0;

	for (; Index < InString.Len(); ++Index)
	{
		// Skip leading zeros
		const TCHAR Current = InString[Index];
		if (!FChar::IsDigit(Current))
		{
			break;
		}

		const uint32 ThisNumber = (Current - '0');

		// Skip leading zeros
		if (Parsed == 0 && ThisNumber == 0)
		{
			continue;
		}

		auto IsValidOperation = [](uint32 Start, uint32 Now) { return Now >= Start && Now < static_cast<uint32>(std::numeric_limits<int32>::max()); };

		// Multiply by 10 using bit-shift trick (x*10 === x*2 + x*8 === x<<1 + x<<3) and check for overflow
		const uint32 Value   = ThisNumber * Factor;
		const uint32 Shifted = Parsed * 10;

		Factor *= 10;

		if (Shifted < Parsed || Shifted+Value < Parsed)
		{
			// Overflow - can't be a uint
			return false;
		}

		Parsed = Shifted+Value;
	}

	if (Index != 0)
	{
		OutResult = Parsed;
		if (OutNumCharsParsed)
		{
			*OutNumCharsParsed = Index;
		}
	}
	return Index != 0;
}


} // namespace UE::UniversalObjectLocator