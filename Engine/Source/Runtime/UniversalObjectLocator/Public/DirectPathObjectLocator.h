// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/SoftObjectPath.h"
#include "UniversalObjectLocatorFwd.h"
#include "DirectPathObjectLocator.generated.h"

class UObject;

/**
 * Object locator type that simply references its object by a direct path
 */
USTRUCT()
struct FDirectPathObjectLocator
{
	GENERATED_BODY()

	UPROPERTY()
	FSoftObjectPath Path;

	UNIVERSALOBJECTLOCATOR_API static UE::UniversalObjectLocator::TFragmentTypeHandle<FDirectPathObjectLocator> FragmentType;

	friend uint32 GetTypeHash(const FDirectPathObjectLocator& A)
	{
		return GetTypeHash(A.Path);
	}

	friend bool operator==(const FDirectPathObjectLocator& A, const FDirectPathObjectLocator& B)
	{
		return A.Path == B.Path;
	}

	UE::UniversalObjectLocator::FResolveResult Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const;
	UE::UniversalObjectLocator::FInitializeResult Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams);

	void ToString(FStringBuilderBase& OutStringBuilder) const;
	UE::UniversalObjectLocator::FParseStringResult TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params);

	static uint32 ComputePriority(const UObject* Object, const UObject* Context);
};

template<>
struct TStructOpsTypeTraits<FDirectPathObjectLocator> : public TStructOpsTypeTraitsBase2<FDirectPathObjectLocator>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};
