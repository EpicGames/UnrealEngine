// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "UniversalObjectLocatorFwd.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocatorFragmentType.h"
#include "UniversalObjectLocatorStringParams.h"
#include "UniversalObjectLocatorInitializeResult.h"
#include "UniversalObjectLocatorFragment.h"
#include "UniversalObjectLocatorFragmentType.h"
#include "Concepts/GetTypeHashable.h"
#include "Templates/UnrealTypeTraits.h"
#include "Misc/GeneratedTypeName.h"

struct FSubObjectLocator;

namespace UE::UniversalObjectLocator
{

struct FFragmentTypeParameters
{
	FFragmentTypeParameters(FName InFragmentTypeID, FText InDisplayText)
		: DisplayText(InDisplayText)
		, FragmentTypeID(InFragmentTypeID)
		, Flags(EFragmentTypeFlags::None)
	{
	}

	FText DisplayText;
	FName FragmentTypeID;
	FName PrimaryEditorType;
	EFragmentTypeFlags Flags;
};

class IUniversalObjectLocatorModule
	: public IModuleInterface
{
public:


	/**
	 * Register a new fragment type that can be used to locate objects from a Universal Object Locator
	 *
	 * Fragment types must be serializable USTRUCTS, must be equality comparable, and must be GetTypeHashable.
	 * Additionally, the following member functions must be defined:
	 *
	 *   // Initialize the fragment to point at an object within a context
	 *   FInitializeResult Initialize(const FInitializeParams& InParams) const;
	 *   // Resolve the fragment to the resulting object, potentially loading or unloading it
	 *   FResolveResult Resolve(const FResolveParams& Params) const;
	 *
	 *   // Convert this fragment to its string representation
	 *   void ToString(FStringBuilderBase& OutStringBuilder) const;
	 *   // Try and parse this fragment from its string representation
	 *   FParseStringResult TryParseString(FStringView InString, const FParseStringParams& Params) const;
	 *
	 *   // Compute a priority for this fragment based on an object and context in order to decide which fragment type should be used if multiple can address the same object
	 *   static uint32 ComputePriority(const UObject* Object, const UObject* Context) const;
	 */
	template<typename PayloadStructType
		UE_REQUIRES(
			// Payloads must be equality-comparable (we don't use the CEqualityComparable concept here because we need to make sure the ICppStructOps are set up correctly)
			(TStructOpsTypeTraits<PayloadStructType>::WithIdenticalViaEquality || TStructOpsTypeTraits<PayloadStructType>::WithIdentical) &&
			// Payloads must be type-hashable
			TModels_V<CGetTypeHashable, PayloadStructType>
		)
	>
	TFragmentTypeHandle<PayloadStructType> RegisterFragmentType(FFragmentTypeParameters& FragmentTypeParameters)
	{
		checkf(FragmentTypeParameters.FragmentTypeID != NAME_None, TEXT("'None' is not a valid Fragment Type ID for type %s"), GetGeneratedTypeName<PayloadStructType>());
		checkf(!FAsciiSet::HasAny(*FragmentTypeParameters.FragmentTypeID.ToString(), ~FUniversalObjectLocatorFragment::ValidFragmentTypeCharacters), TEXT("Fragment Type ID '%s' contains invalid characters"), *FragmentTypeParameters.FragmentTypeID.ToString());

		FFragmentType NewFragmentType;
		NewFragmentType.FragmentTypeID     = FragmentTypeParameters.FragmentTypeID;
		NewFragmentType.PrimaryEditorType  = FragmentTypeParameters.PrimaryEditorType;
		NewFragmentType.Flags              = FragmentTypeParameters.Flags;
		NewFragmentType.PayloadType        = PayloadStructType::StaticStruct();

		// Static bindings
		NewFragmentType.StaticBindings.Priority = PayloadStructType::ComputePriority;

		// Member function bindings
		NewFragmentType.InstanceBindings.Resolve = [](const void* Payload, const FResolveParams& Params) -> FResolveResult
		{
			return static_cast<const PayloadStructType*>(Payload)->Resolve(Params);
		};
		NewFragmentType.InstanceBindings.Initialize = [](void* Payload, const FInitializeParams& InParams) -> FInitializeResult
		{
			return static_cast<PayloadStructType*>(Payload)->Initialize(InParams);
		};
		NewFragmentType.InstanceBindings.ToString = [](const void* Payload, FStringBuilderBase& OutStringBuilder)
		{
			return static_cast<const PayloadStructType*>(Payload)->ToString(OutStringBuilder);
		};
		NewFragmentType.InstanceBindings.TryParseString = [](void* Payload, FStringView InString, const FParseStringParams& Params) -> FParseStringResult
		{
			return static_cast<PayloadStructType*>(Payload)->TryParseString(InString, Params);
		};
#if UE_UNIVERSALOBJECTLOCATOR_DEBUG
		NewFragmentType.StaticBindings.FragmentDebugInitializer = [](void* Payload)
		{
			new (Payload) TFragmentPayload<PayloadStructType>;
		};
#endif
		return TFragmentTypeHandle<PayloadStructType>(RegisterFragmentTypeImpl(NewFragmentType));
	}

	template<typename ParameterType>
	TParameterTypeHandle<ParameterType> RegisterParameterType()
	{
		return TParameterTypeHandle<ParameterType>(RegisterParameterTypeImpl(ParameterType::StaticStruct()));
	}

	void UnregisterParameterType(FParameterTypeHandle InHandle)
	{
		UnregisterParameterTypeImpl(InHandle);
	}

	template<typename PayloadStructType>
	void UnregisterFragmentType(TFragmentTypeHandle<PayloadStructType> FragmentType)
	{
		UnregisterFragmentTypeImpl(FragmentType);
	}

private:

	virtual FFragmentTypeHandle RegisterFragmentTypeImpl(const FFragmentType& FragmentType) = 0;
	virtual void UnregisterFragmentTypeImpl(FFragmentTypeHandle FragmentType) = 0;

	virtual FParameterTypeHandle RegisterParameterTypeImpl(UScriptStruct* Struct) = 0;
	virtual void UnregisterParameterTypeImpl(FParameterTypeHandle ParameterType) = 0;
};


} // namespace UE::UniversalObjectLocator