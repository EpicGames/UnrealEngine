// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/SoftObjectPath.h"
#include "UniversalObjectLocatorFwd.h"
#include "SubObjectLocator.generated.h"

class UObject;

USTRUCT()
struct FSubObjectLocator
{
	GENERATED_BODY()

	UPROPERTY()
	FString PathWithinContext;

	UNIVERSALOBJECTLOCATOR_API static UE::UniversalObjectLocator::TFragmentTypeHandle<FSubObjectLocator> FragmentType;

	friend uint32 GetTypeHash(const FSubObjectLocator& A)
	{
		return GetTypeHash(A.PathWithinContext);
	}

	friend bool operator==(const FSubObjectLocator& A, const FSubObjectLocator& B)
	{
		return A.PathWithinContext == B.PathWithinContext;
	}

	UE::UniversalObjectLocator::FResolveResult Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const;
	UE::UniversalObjectLocator::FInitializeResult Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams);
	void ToString(FStringBuilderBase& OutStringBuilder) const;
	UE::UniversalObjectLocator::FParseStringResult TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params);

	static uint32 ComputePriority(const UObject* Object, const UObject* Context);
};

template<>
struct TStructOpsTypeTraits<FSubObjectLocator> : public TStructOpsTypeTraitsBase2<FSubObjectLocator>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};
