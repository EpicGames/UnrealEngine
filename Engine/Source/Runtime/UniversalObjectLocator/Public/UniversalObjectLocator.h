// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UniversalObjectLocatorFwd.h"
#include "UniversalObjectLocatorFragment.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "UniversalObjectLocatorFragmentType.h"
#include "UniversalObjectLocator.generated.h"

namespace UE::UniversalObjectLocator
{
	struct FUniversalObjectLocatorCustomization;
	struct FFragmentItem;
}

/**
 * Universal Object Locators (UOLs) define an address to an object.
 *
 * The address is implemented as a chain of FUniversalObjectLocatorFragments, allowing addressing of objects
 *   that may be nested deeply within levels of externally defined spawn or ownership logic.
 *
 * For example, a Universal Object Locator may reference an Anim Instance within a Skeletal Mesh Actor
 *   is spawned by a Child Actor Component that is spawned by Sequencer. This is impossible with a
 *   regular soft object path, but is perfectly feasible for a UOL.
 *
 * This type is 16 bytes.
 */
USTRUCT(BlueprintType, Category=GameFramework, meta=(HasNativeMake="/Script/Engine.UniversalObjectLocatorScriptingExtensions.MakeUniversalObjectLocator"))
struct FUniversalObjectLocator
{
	GENERATED_BODY()

	using FResolveParams       = UE::UniversalObjectLocator::FResolveParams;
	using FResolveResult       = UE::UniversalObjectLocator::FResolveResult;
	using FResolveResultData   = UE::UniversalObjectLocator::FResolveResultData;
	using FParseStringParams   = UE::UniversalObjectLocator::FParseStringParams;
	using FParseStringResult   = UE::UniversalObjectLocator::FParseStringResult;

	/**
	 * Default constructor
	 */
	UNIVERSALOBJECTLOCATOR_API FUniversalObjectLocator();

	/**
	 * Attempt to construct this locator from a given object.
	 * May result in an empty locator if no suitable address could be created.
	 *
	 * @param Object           The object that this locator should represent
	 * @param Context          (Optional) Constrain this universal reference based on the specified context. This context should be passed to Resolve otherwise the resolution may fail.
	 * @param StopAtContext    (Optional) Stop constructing this universal reference when we reach the following context (can be used if that context is always passed to Resolve to keep this type smaller)
	 */
	UNIVERSALOBJECTLOCATOR_API FUniversalObjectLocator(UObject* Object, UObject* Context = nullptr, UObject* StopAtContext = nullptr);

	/**
	 * Check if this locator is 'empty'. An empty locator contains no fragments and will never resolve.
	 */
	bool IsEmpty() const
	{
		return Fragments.Num() == 0;
	}

	/**
	 * Attempt to resolve this locator by invoking the payload's 'Resolve' function
	 *
	 * @param Params           Resolution parameters defining the type of resolution to perform
	 * @return A result structure defining the resolved object pointer, and associated flags
	 */
	UNIVERSALOBJECTLOCATOR_API FResolveResult Resolve(const FResolveParams& Params) const;

	/**
	 * Attempt to find the object this locator points to.
	 * Shorthand for Resolve(FResolveParams::AsyncFind(Context)).AsObject().ReleaseFuture().
	 *
	 * @param Context          (Optional) An optional context to use for finding the object - should match what was specified in Reset() or in construction
	 * @return A result structure that may or may not already have a result populated
	 */
	UNIVERSALOBJECTLOCATOR_API FResolveResult AsyncFind(UObject* Context = nullptr) const;

	/**
	 * Attempt to find the object this locator points to, loading it if necessary (and possible).
	 * Shorthand for Resolve(FResolveParams::AsyncLoad(Context)).AsObject().ReleaseFuture().
	 *
	 * @param Context          (Optional) An optional context to use for finding/loading the object - should match what was specified in Reset()
	 * @return A result structure that may or may not already have a result populated
	 */
	UNIVERSALOBJECTLOCATOR_API FResolveResult AsyncLoad(UObject* Context = nullptr) const;

	/**
	 * Attempt to unload the object this locator points to if possible.
	 * Shorthand for Resolve(FResolveParams::AsyncUnload(Context)).AsVoid().ReleaseFuture().
	 *
	 * @param Context          (Optional) An optional context to use for finding/loading the object - should match what was specified in Reset()
	 * @return A result structure that may or may not already have a result populated
	 */
	UNIVERSALOBJECTLOCATOR_API FResolveResult AsyncUnload(UObject* Context = nullptr) const;

	/**
	 * Attempt to find the object this locator points to.
	 * Shorthand for Resolve(FResolveParams::SyncFind(Context)).AsObject().SyncGet().Object.
	 *
	 * @param Context          (Optional) An optional context to use for finding the object - should match what was specified in Reset() or in construction
	 * @return The located object, or nullptr on failure
	 */
	UNIVERSALOBJECTLOCATOR_API UObject* SyncFind(UObject* Context = nullptr) const;

	/**
	 * Attempt to find the object this locator points to, loading it if necessary (and possible), and blocking until it is loaded.
	 * Shorthand for Resolve(FResolveParams::SyncLoad(Context)).AsObject().SyncGet().Object.
	 *
	 * @param Context          (Optional) An optional context to use for finding/loading the object - should match what was specified in Reset()
	 * @return The located object, or nullptr on failure
	 */
	UNIVERSALOBJECTLOCATOR_API UObject* SyncLoad(UObject* Context = nullptr) const;

	/**
	 * Attempt to unload the object this locator points to if possible.
	 * Shorthand for Resolve(FResolveParams::SyncUnload(Context)).AsVoid().SyncGet().
	 *
	 * @param Context          (Optional) An optional context to use for finding/loading the object - should match what was specified in Reset()
	 * @return The located object, or nullptr on failure
	 */
	UNIVERSALOBJECTLOCATOR_API void SyncUnload(UObject* Context = nullptr) const;

	/**
	 * Retrieve the fragment type relating to the last locator in this address
	 */
	UNIVERSALOBJECTLOCATOR_API const UE::UniversalObjectLocator::FFragmentType* GetLastFragmentType() const;

	/**
	 * Retrieve the fragment type handle relating to the last locator in this address
	 */
	UNIVERSALOBJECTLOCATOR_API UE::UniversalObjectLocator::FFragmentTypeHandle GetLastFragmentTypeHandle() const;

	/**
	 * Convert this locator to its string representation
	 *
	 * @param OutString        String builder to populate
	 */
	UNIVERSALOBJECTLOCATOR_API void ToString(FStringBuilderBase& OutString) const;

	/**
	 * Attempt to deserialize this locator from a string
	 *
	 * @param InString         The string to parse
	 * @param InParams         Additional string parameters
	 * @return Parse result, specifying success or failure, and number of characters that were parsed
	 */
	UNIVERSALOBJECTLOCATOR_API FParseStringResult TryParseString(FStringView InString, const FParseStringParams& InParams);

	/**
	 * Attempt to deserialize a new locator from a string.
	 * Shorthand for FUniversalObjectLocator L; L.TryParseString(InString, InParams);.
	 *
	 * @param InString         The string to parse
	 * @param InParams         Additional string parameters
	 * @return A (perhaps empty) object locator
	 */
	static UNIVERSALOBJECTLOCATOR_API FUniversalObjectLocator FromString(FStringView InString, const FParseStringParams& InParams);

public:

	/**
	 * Reset this locator back to its default-constructed, empty state
	 */
	UNIVERSALOBJECTLOCATOR_API void Reset();

	/**
	 * Attempt to reset this locator to point at a different object.
	 * May result in an empty locator if no suitable address could be created.
	 *
	 * @param Object           The object that this locator should represent
	 * @param Context          (Optional) Constrain this universal reference based on the specified context. This context should be passed to Resolve otherwise the resolution may fail.
	 * @param StopAtContext    (Optional) Stop constructing this universal reference when we reach the following context (can be used if that context is always passed to Resolve to keep this type smaller)
	 */
	UNIVERSALOBJECTLOCATOR_API void Reset(UObject* Object, UObject* Context = nullptr, UObject* StopAtContext = nullptr);

	/**
	 * Add a fragment to the end of this locator
	 *
	 * @param InFragment       The fragment to add
	 */
	UNIVERSALOBJECTLOCATOR_API void AddFragment(FUniversalObjectLocatorFragment&& InFragment);

	/**
	 * Templated helper for AddFragment
	 *
	 * @param InFragment       The fragment to add
	 */
	template<typename FragmentType, typename ...ArgTypes>
	void AddFragment(ArgTypes&&... FragmentArgs);

	/**
	 * Retrieve the last fragment in this address
	 */
	UNIVERSALOBJECTLOCATOR_API FUniversalObjectLocatorFragment* GetLastFragment();

	/**
	 * Retrieve the last fragment in this address
	 */
	UNIVERSALOBJECTLOCATOR_API const FUniversalObjectLocatorFragment* GetLastFragment() const;

	/**
	 * Iterate the fragments in this address
	 * @param InFunction       The visitor function to call for each fragment in this address. Function should return false to stop iteration.
	 * @return false if the iteration was stopped early, true if the iteration succeeded (or if empty)
	 */
	UNIVERSALOBJECTLOCATOR_API bool ForEachFragment(TFunctionRef<bool(int32, int32, const FUniversalObjectLocatorFragment&)> InFunction) const;

	/*
	* Iterates over all fragments and combines their types' default flags.
	*/
	UNIVERSALOBJECTLOCATOR_API UE::UniversalObjectLocator::EFragmentTypeFlags GetDefaultFlags() const;

	/**
	 * Equality comparison.
	 * @note: This tests for exact equality of its piecewise fragments. Equivalent but not equal locators will always return false.
	 */
	UNIVERSALOBJECTLOCATOR_API friend bool operator==(const FUniversalObjectLocator& A, const FUniversalObjectLocator& B);

	/**
	 * Inequality comparison.
	 * @note: This tests for exact inequality of its piecewise fragments. Equivalent but not equal locators will always return true.
	 */
	UNIVERSALOBJECTLOCATOR_API friend bool operator!=(const FUniversalObjectLocator& A, const FUniversalObjectLocator& B);

	/**
	 * Type hashable
	 */
	UNIVERSALOBJECTLOCATOR_API friend uint32 GetTypeHash(const FUniversalObjectLocator& Locator);

	/*~ Begin TStructOpsTypeTraits implementation */
	UNIVERSALOBJECTLOCATOR_API bool ExportTextItem(FString& ValueStr, const FUniversalObjectLocator& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	UNIVERSALOBJECTLOCATOR_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText, FArchive* InSerializingArchive = nullptr);
	UNIVERSALOBJECTLOCATOR_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
	/*~ End TStructOpsTypeTraits implementation */

private:
	/** Adds the 'best' fragment according to the input object and context, as well as any potential required relative fragments */
	UNIVERSALOBJECTLOCATOR_API bool AddFragment(const UObject* Object, UObject* Context, UObject* StopAtContext);

	FResolveResult ResolveSyncImpl(const FResolveParams& Params) const;
	FResolveResult ResolveAsyncImpl(const FResolveParams& Params) const;

private:

	/** Array of relative locators ordered sequentially from outer to inner. The first locator is probably 'absolute' and is resolved with no context, although that is not a hard restriction */
	UPROPERTY()
	TArray<FUniversalObjectLocatorFragment> Fragments;

	friend struct UE::UniversalObjectLocator::FUniversalObjectLocatorCustomization;
	friend struct UE::UniversalObjectLocator::FFragmentItem;
};

template<typename FragmentType, typename ...ArgTypes>
void FUniversalObjectLocator::AddFragment(ArgTypes&&... FragmentArgs)
{
	TUniversalObjectLocatorFragment<FragmentType> NewFragment(Forward<ArgTypes>(FragmentArgs)...);
	AddFragment(MoveTemp(NewFragment));
}

template<>
struct TStructOpsTypeTraits<FUniversalObjectLocator> : public TStructOpsTypeTraitsBase2<FUniversalObjectLocator>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithExportTextItem = true,
		WithImportTextItem = true,
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

