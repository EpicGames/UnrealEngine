// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Class.h"
#include "UniversalObjectLocatorFwd.h"
#include "UniversalObjectLocatorFragmentType.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocatorFragmentDebugging.h"
#include "Templates/Function.h"
#include "UniversalObjectLocatorFragment.generated.h"


/**
 * Universal Object Locator (UOL) Fragments provide an extensible mechanism for referencing permanent, transient
 *   or dynamically created objects relative to an external context. UOLs comprise zero or more nested fragments.
 * 
 * Creation and resolution of a fragment requires a context to be provided;
 *   normally this will be the object on which the UOL exists as a property.
 *
 * The way in which the object is referenced is defined by globally registered 'FragmentTypes' 
 *   (See IUniversalObjectLocatorModule::RegisterFragmentType). Each FragmentType can be thought of as somewhat
 *   equivalent to a www URI fragment type, though the 'path' is not necessarily just a string, but includes
 *   support for the full set of Engine Property types.
 * 
 * The type is implemented as a type-erased payload block, a fragment type handle and some internal flags.
 * Payloads will be allocated using the inline memory if alignment and size constraints allow, but
 *   will fall back to a heap allocation if necessary. Allocation should be avoided by keeping payload
 *   types small.
 *
 * Aligned to 8 bytes, 32 (runtime) or 64 (editor) bytes big.
 */
USTRUCT(BlueprintType)
struct alignas(8) FUniversalObjectLocatorFragment
{
	GENERATED_BODY()

	using FParseStringResult = UE::UniversalObjectLocator::FParseStringResult;
	using FParseStringParams = UE::UniversalObjectLocator::FParseStringParams;

	static inline constexpr FAsciiSet ValidFragmentTypeCharacters = "abcdefghijklmnopqrstuvwxyzABCDEFGHIJKLMNOPQRSTUVWXYZ0123456789_-";
	// Valid fragment delimiters are those that are allowable by RFC3986 for the query part (unreserved / pct-encoded / sub-delims / ":" / "@" / "/" / "?") excluding '&' which we use to separate fragments within a path
	static inline constexpr FAsciiSet ValidFragmentDelimiters = "%!$'()*+,;=/?:@.~";
	static inline constexpr FAsciiSet ValidFragmentPayloadCharacters = ValidFragmentTypeCharacters | ValidFragmentDelimiters;

	/** Make our inline data buffer larger in-editor to support editor-only data without allocation */
#if WITH_EDITORONLY_DATA
	static constexpr SIZE_T SizeInMemory = 64;
#else
	static constexpr SIZE_T SizeInMemory = 32;
#endif


	/**
	 * Construct a new fragment with a specific fragment type and data.
	 * Used when a specific type of relative fragment is required.
	 *
	 * @param InHandle    A typed handle to the fragment type to use for construction. Retrieved from IUniversalObjectLocatorModule::RegisterFragmentType.
	 * @param InArgs      (Optional) Payload construction arguments to be passed to T on construction. Omission implies default construction.
	 */
	template<typename T, typename ...ArgTypes>
	FUniversalObjectLocatorFragment(UE::UniversalObjectLocator::TFragmentTypeHandle<T> InHandle, ArgTypes&& ...InArgs);


	/**
	 * Construct a new fragment with a specific fragment type and default-constructed payload.
	 * Used when a specific type of relative fragment is required.
	 *
	 * @param InFragmentType  The fragment type to use with this fragment
	 */
	UNIVERSALOBJECTLOCATOR_API FUniversalObjectLocatorFragment(const UE::UniversalObjectLocator::FFragmentType& InFragmentType);

	/**
	 * Construct this fragment by binding it to an object within a given context.
	 * @note: This constructor can 'fail' and result in an Empty fragment if no suitable fragment type could be found for the object
	 */
	UNIVERSALOBJECTLOCATOR_API FUniversalObjectLocatorFragment(const UObject* InObject, UObject* Context);

	/** Default constructor: initializes to an empty fragment with no fragment type */
	UNIVERSALOBJECTLOCATOR_API FUniversalObjectLocatorFragment();

	/** Destructor - destructs the payload, and frees any heap allocation as necessary */
	UNIVERSALOBJECTLOCATOR_API ~FUniversalObjectLocatorFragment();

	/** Copy construction/assignment */
	UNIVERSALOBJECTLOCATOR_API FUniversalObjectLocatorFragment(const FUniversalObjectLocatorFragment& RHS);
	UNIVERSALOBJECTLOCATOR_API FUniversalObjectLocatorFragment& operator=(const FUniversalObjectLocatorFragment& RHS);

	/** Move construction/assignment */
	UNIVERSALOBJECTLOCATOR_API FUniversalObjectLocatorFragment(FUniversalObjectLocatorFragment&& RHS);
	UNIVERSALOBJECTLOCATOR_API FUniversalObjectLocatorFragment& operator=(FUniversalObjectLocatorFragment&& RHS);

	/** Equality comparison. Compares the fragment type and payload data. */
	UNIVERSALOBJECTLOCATOR_API friend bool operator==(const FUniversalObjectLocatorFragment& A, const FUniversalObjectLocatorFragment& B);
	/** Inequality comparison. Compares the fragment type and payload data. */
	UNIVERSALOBJECTLOCATOR_API friend bool operator!=(const FUniversalObjectLocatorFragment& A, const FUniversalObjectLocatorFragment& B);
	/** Type hashable */
	UNIVERSALOBJECTLOCATOR_API friend uint32 GetTypeHash(const FUniversalObjectLocatorFragment& Fragment);
public:

	/**
	 * Attempt to resolve this fragment by invoking the payload's 'Resolve' function
	 *
	 * @param Params           Resolution parameters, defining the context to resolve within, and the type of resolution to perform
	 * @return A result structure defining the resolved object pointer, and associated flags
	 */
	UNIVERSALOBJECTLOCATOR_API UE::UniversalObjectLocator::FResolveResult Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const;

	/**
	 * Check whether this reference is empty.
	 * @note: An empty fragment can never resolve to an object, but is distinct from, and not equal to, a populated fragment that does points to a non-existent or irretrievable object.
	 */
	bool IsEmpty() const
	{
		return !FragmentType.IsValid();
	}

public:

	/**
	 * Reset this fragment back to its default-constructed, empty state
	 */
	UNIVERSALOBJECTLOCATOR_API void Reset();

	/**
	 * Reset this fragment to point to a new object from the specified context
	 * @note: If no suitable fragment type could be found for the object and context, results in an Empty fragment
	 */
	UNIVERSALOBJECTLOCATOR_API void Reset(const UObject* InObject, UObject* Context);

	/**
	 * Reset this fragment to point to a new object from the specified context using a filtered set of fragment types
	 * @note: If no suitable fragment type could be found for the object and context, results in an Empty fragment
	 */
	UNIVERSALOBJECTLOCATOR_API void Reset(const UObject* InObject, UObject* Context, TFunctionRef<bool(UE::UniversalObjectLocator::FFragmentTypeHandle)> CanUseFragmentType);

public:

	/**
	 * Convert this fragment to a string of the form fragment-id[=fragment-payload]
	 *
	 * @param OutString        String builder to populate
	 */
	UNIVERSALOBJECTLOCATOR_API void ToString(FStringBuilderBase& OutString) const;

	/**
	 * Attempt to initialize this fragment from a string of the form fragment-type-id[=payload]
	 * The state of this instance will not be changed if this function returns false.
	 *
	 * @param InString         The string to parse.
	 * @param InParams         Additional string parameters
	 * @return Parse result, specifying success or failure, and number of characters that were parsed
	 */
	UNIVERSALOBJECTLOCATOR_API FParseStringResult TryParseString(FStringView InString, const FParseStringParams& InParams);

	/**
	 * Attempt to default initialize this fragment using a string that defines the type
	 * The state of this instance will not be changed if this function returns false.
	 *
	 * @param InString         The string to parse.
	 * @param InParams         Additional string parameters
	 * @return Parse result, specifying success or failure, and number of characters that were parsed
	 */
	UNIVERSALOBJECTLOCATOR_API FParseStringResult TryParseFragmentType(FStringView InString, const FParseStringParams& InParams);

	/**
	 * Attempt to deserialize this fragment's payload from a string, based on its currently assigned type
	 * The state of this instance will not be changed if this function returns false.
	 *
	 * @param InString         The string to parse.
	 * @param InParams         Additional string parameters
	 * @return Parse result, specifying success or failure, and number of characters that were parsed
	 */
	UNIVERSALOBJECTLOCATOR_API FParseStringResult TryParseFragmentPayload(FStringView InString, const FParseStringParams& InParams);

public:

	/**
	 * Try and retrieve this fragment's payload as a specific type using its fragment type handle
	 *
	 * @param InType           The handle of the fragment type to this payload to. This is returned from IUniversalObjectLocatorModule::RegisterFragmentType on registration.
	 * @return A mutable pointer to the payload, or nullptr if this fragment is empty or of a different type.
	 */
	template<typename T>
	T* GetPayloadAs(UE::UniversalObjectLocator::TFragmentTypeHandle<T> InType);

	/**
	 * Try and retrieve this fragment's payload as a specific type using its fragment type handle
	 *
	 * @param InType           The handle of the fragment type to this payload to. This is returned from IUniversalObjectLocatorModule::RegisterFragmentType on registration.
	 * @return A mutable pointer to the payload, or nullptr if this fragment is empty or of a different type.
	 */
	template<typename T>
	const T* GetPayloadAs(UE::UniversalObjectLocator::TFragmentTypeHandle<T> InType) const;

	/**
	 * Try and retrieve this fragment's payload as a specific type using its fragment type handle
	 *
	 * @param InType           The handle of the fragment type to this payload to. This is returned from IUniversalObjectLocatorModule::RegisterFragmentType on registration.
	 * @param OutData          Pointer to retrieve the resulting payload ptr
	 * @return true on success, false if this fragment is empty or of a different type.
	 */
	template<typename T>
	bool TryGetPayloadAs(UE::UniversalObjectLocator::TFragmentTypeHandle<T> InType, T*& OutData);

	/**
	 * Try and retrieve this fragment's payload as a specific type using its fragment type handle
	 *
	 * @param InType           The handle of the fragment type to this payload to. This is returned from IUniversalObjectLocatorModule::RegisterFragmentType on registration.
	 * @param OutData          Pointer to retrieve the resulting payload ptr
	 * @return true on success, false if this fragment is empty or of a different type.
	 */
	template<typename T>
	bool TryGetPayloadAs(UE::UniversalObjectLocator::TFragmentTypeHandle<T> InType, const T*& OutData) const;

	/**
	 * Retrieve this fragment's payload data
	 *
	 * @return The payload data or nullptr if this fragment is empty
	 */
	UNIVERSALOBJECTLOCATOR_API void* GetPayload();

	/**
	 * Retrieve this fragment's payload data
	 *
	 * @return The payload data or nullptr if this fragment is empty
	 */
	UNIVERSALOBJECTLOCATOR_API const void* GetPayload() const;

	/**
	 * Retrieve this fragment's fragment type
	 *
	 * @return The fragment type or nullptr if this fragment is empty
	 */
	UNIVERSALOBJECTLOCATOR_API const UE::UniversalObjectLocator::FFragmentType* GetFragmentType() const;

	/**
	 * Retrieve this fragment's fragment struct type
	 *
	 * @return The fragment struct type or nullptr if this fragment is empty
	 */
	UNIVERSALOBJECTLOCATOR_API UScriptStruct* GetFragmentStruct() const;

	/**
	 * Retrieve this fragment's fragment type handle
	 *
	 * @return The fragment type handle (possibly invalid if this fragment is empty)
	 */
	UNIVERSALOBJECTLOCATOR_API UE::UniversalObjectLocator::FFragmentTypeHandle GetFragmentTypeHandle() const;

public:

	/*~ Begin TStructOpsTypeTraits implementation */
	UNIVERSALOBJECTLOCATOR_API bool Serialize(FArchive& Ar);
	UNIVERSALOBJECTLOCATOR_API void AddStructReferencedObjects(FReferenceCollector& Collector);
	UNIVERSALOBJECTLOCATOR_API bool ExportTextItem(FString& ValueStr, const FUniversalObjectLocatorFragment& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	UNIVERSALOBJECTLOCATOR_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText, FArchive* InSerializingArchive = nullptr);
	UNIVERSALOBJECTLOCATOR_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
	UNIVERSALOBJECTLOCATOR_API void GetPreloadDependencies(TArray<UObject*>& OutDeps);
	/*~ End TStructOpsTypeTraits implementation */

protected:


	/** Runtime type-checking against specific struct types. Compiled out if checks are not enabled. */
#if DO_CHECK
	/** Runtime type-checking against a specific struct type. Compiled out if checks are not enabled. */
	UNIVERSALOBJECTLOCATOR_API void CheckPayloadType(UScriptStruct* TypeToCompare) const;
#else
	FORCEINLINE static constexpr void CheckPayloadType(void* TypeToCompare)
	{
	}
#endif

protected:

	/**
	 * Describes a UOL fragment payload after it has been allocated
	 * Where UE_UNIVERSALOBJECTLOCATOR_DEBUG is enabled, the allocation will be laid out as such:
	 * 0..								8..															8+sizeof(T)
	 * [TFragmentPayload<T>				| T Payload													]
	 *
	 * When UE_UNIVERSALOBJECTLOCATOR_DEBUG is disabled, the allocation is simply:
	 * 0..															sizeof(T)
	 * | T Payload													]
	 */
	struct FAllocatedPayload
	{
#if UE_UNIVERSALOBJECTLOCATOR_DEBUG
		/** Pointer to artificially inserted vtable ptr that identifies the fragment data polymorphically */
		void* DebugVFTablePtr;
#endif
		/** Pointer to the actual typed fragment data */
		void* Payload;
	};

	/**
	 * Allocate (but do not initialize) the fragment payload using the specified size and alignment,
	 *     accounting for enough space to fit a debug vftable ptr whern UE_UNIVERSALOBJECTLOCATOR_DEBUG is enabled.
	 * 
	 * @return Structure identifying the payload
	 */
	UNIVERSALOBJECTLOCATOR_API FAllocatedPayload AllocatePayload(size_t Size, size_t Alignment);

	/**
	 * Default-initialize the fragment payload using the specified type
	 */
	void DefaultConstructPayload(const UE::UniversalObjectLocator::FFragmentType& InFragmentType);

	/**
	 * Destroy the payload (if valid) by calling its destructor and freeing the memory (if necessary)
	 */
	void DestroyPayload();

private:

	uint32 GetDebugHeaderOffset() const
	{
		// Special case for DebugHeaderSizeLog2==0 which signifies no offset rather than 2^0 = 1 byte.
		// We do this using a branchless bitmask that always unsets the first bit (which can never be set, because we always return a power of 2 > 1)
		return (1ul << DebugHeaderSizeLog2) & (~1ul);
	}

#if UE_UNIVERSALOBJECTLOCATOR_DEBUG
	/*~ Utility symbol name to guarantee that FFragmentType can be resolved within the context of a FUniversalObjectLocatorFragment within natvis expressions */
	struct FDebuggableFragmentType
	{
		using Type = UE::UniversalObjectLocator::FFragmentType;
	};
	struct FDebuggableFragment
	{
		using Type = UE::UniversalObjectLocator::IFragmentPayload;
	};
#endif

	/*
	 * Payload data - implicitly aligned to a 8 byte boundary since it's the first member.
	 * Given payload type T, this is either a type-erased T() value (where bIsInline==1),
	 *    or a T* to a heap allocated T (where bIsInline==0)
	 * SizeInMemory is specifically defined by the desired overall size of FUniversalObjectLocatorFragment::SizeInMemory, minus space for other members
	 */
	uint8 Data[SizeInMemory-2];

	/** 1 Byte - the fragment type portion of the universal fragment */
	UE::UniversalObjectLocator::FFragmentTypeHandle FragmentType;

	/*~ 1 Byte of flags */
	/** True when FragmentType has been assigned to a valid handle, and Data has been initialized with FragmentType::PayloadType */
	uint8 bIsInitialized : 1;
	/** True if Data is an inline allocation of FragmentType::PayloadType, false means Data is a (void*) to the heap allocated data. */
	uint8 bIsInline : 1;
	/** Offset from the allocated memory to the fragment payload stored as a power of 2. Only non-zero when UE_UNIVERSALOBJECTLOCATOR_DEBUG is enabled. */
	uint8 DebugHeaderSizeLog2 : 6;
};

template<>
struct TStructOpsTypeTraits<FUniversalObjectLocatorFragment> : public TStructOpsTypeTraitsBase2<FUniversalObjectLocatorFragment>
{
	enum
	{
		WithSerializer = true,
		WithCopy = true,
		WithIdenticalViaEquality = true,
		WithExportTextItem = true,
		WithImportTextItem = true,
		WithAddStructReferencedObjects = true,
	};
};


template<typename T, typename ...ArgTypes>
FUniversalObjectLocatorFragment::FUniversalObjectLocatorFragment(UE::UniversalObjectLocator::TFragmentTypeHandle<T> InHandle, ArgTypes&& ...InArgs)
	: FragmentType(InHandle)
	, bIsInitialized(0)
	, DebugHeaderSizeLog2(0)
{
	using namespace UE::UniversalObjectLocator;

	checkf(InHandle, TEXT("Attempting to construct a new fragment from an invalid fragment type handle - was it registered?"));

	FAllocatedPayload Allocation = AllocatePayload(sizeof(T), alignof(T));

#if UE_UNIVERSALOBJECTLOCATOR_DEBUG
	// Initialize the fragment vftable if necessary. We can do this without needing the fragment type since we know the type
	new (Allocation.DebugVFTablePtr) TFragmentPayload<T>;
#endif

	// Placement new the payload
	new (Allocation.Payload) T{ Forward<ArgTypes>(InArgs)... };
}

template<typename T>
T* FUniversalObjectLocatorFragment::GetPayloadAs(UE::UniversalObjectLocator::TFragmentTypeHandle<T> InType)
{
	if (FragmentType.IsValid() && ensureMsgf(FragmentType == InType, TEXT("Type mismatch when accessing payload data!")))
	{
		return static_cast<T*>(GetPayload());
	}
	return nullptr;
}

template<typename T>
const T* FUniversalObjectLocatorFragment::GetPayloadAs(UE::UniversalObjectLocator::TFragmentTypeHandle<T> InType) const
{
	return const_cast<FUniversalObjectLocatorFragment*>(this)->GetPayloadAs(InType);
}

template<typename T>
bool FUniversalObjectLocatorFragment::TryGetPayloadAs(UE::UniversalObjectLocator::TFragmentTypeHandle<T> InType, T*& OutData)
{
	if (FragmentType.IsValid() && FragmentType == InType)
	{
		OutData = static_cast<T*>(GetPayload());
		return true;
	}
	return false;
}

template<typename T>
bool FUniversalObjectLocatorFragment::TryGetPayloadAs(UE::UniversalObjectLocator::TFragmentTypeHandle<T> InType, const T*& OutData) const
{
	if (FragmentType.IsValid() && FragmentType == InType)
	{
		OutData = static_cast<const T*>(GetPayload());
		return true;
	}
	return false;
}


/** Empty struct type used for deserializing unknown fragment type payloads */
USTRUCT()
struct FUniversalObjectLocatorEmptyPayload
{
	GENERATED_BODY()
};


template<typename PayloadType>
struct TUniversalObjectLocatorFragment : FUniversalObjectLocatorFragment
{
	TUniversalObjectLocatorFragment()
		: FUniversalObjectLocatorFragment(PayloadType::FragmentType)
	{
	}

	TUniversalObjectLocatorFragment(UE::UniversalObjectLocator::TFragmentTypeHandle<PayloadType> InHandle)
		: FUniversalObjectLocatorFragment(InHandle)
	{
	}

	template<typename ...ArgTypes>
	TUniversalObjectLocatorFragment(ArgTypes&& ...InArgs)
		: FUniversalObjectLocatorFragment(PayloadType::FragmentType, Forward<ArgTypes>(InArgs)...)
	{
	}

	template<typename ...ArgTypes>
	TUniversalObjectLocatorFragment(UE::UniversalObjectLocator::TFragmentTypeHandle<PayloadType> InHandle, ArgTypes&& ...InArgs)
		: FUniversalObjectLocatorFragment(InHandle, Forward<ArgTypes>(InArgs)...)
	{
	}

	PayloadType* GetPayload()
	{
		CheckPayloadType(PayloadType::StaticStruct());
		return static_cast<PayloadType*>(FUniversalObjectLocatorFragment::GetPayload());
	}

	const PayloadType* GetPayload() const
	{
		CheckPayloadType(PayloadType::StaticStruct());
		return static_cast<const PayloadType*>(FUniversalObjectLocatorFragment::GetPayload());
	}
};
