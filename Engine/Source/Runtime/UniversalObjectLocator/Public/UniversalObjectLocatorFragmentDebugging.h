// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#ifndef UE_UNIVERSALOBJECTLOCATOR_DEBUG
	#define UE_UNIVERSALOBJECTLOCATOR_DEBUG !UE_BUILD_SHIPPING
#endif

#if UE_UNIVERSALOBJECTLOCATOR_DEBUG

namespace UE::UniversalObjectLocator
{

	/**
	 * Type whose sole purpose is to add a vtable pointer in front of a fragment
	 *      to assist in debugging. When allocating the fragment, a TFragmentPayload<T>
	 *      is allocated in the preceeding 8 bytes which can be used by a natvis expression
	 *      to show the proceeding bytes as a (T*)
	 */
	struct alignas(8) IFragmentPayload
	{
		virtual ~IFragmentPayload()
		{
		}
	};

	/**
	 * Templated version of IFragmentPayload that is added to the start of a fragment.
	 * Utilizes a zero-sized array to cast the proceeding bytes to a T*.
	 */
	template<typename T>
	struct TFragmentPayload : IFragmentPayload
	{
		UE_NO_UNIQUE_ADDRESS uint8 Ptr[];
	};

} // namespace UE::UniversalObjectLocator

#endif // UE_UNIVERSALOBJECTLOCATOR_DEBUG
