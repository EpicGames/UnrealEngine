// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AsciiSet.h"
#include "Misc/StringBuilder.h"
#include "Containers/PagedArray.h"
#include "Containers/StringView.h"

#include "UObject/ObjectPtr.h"
#include "UObject/Class.h"

#include "UniversalObjectLocatorFwd.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "UniversalObjectLocatorFragmentDebugging.h"

namespace UE::UniversalObjectLocator
{

struct IPayloadDebugging;
struct IFragmentTypeDebuggingAssistant;

enum class EFragmentTypeFlags : uint8
{
	None,

	/** Flag to indicate that this fragment type can be loaded, indicating that asset loading or actor spawning will occur if the 'Load' flag
	*   is passed in the FResolveParams. If a fragment type can be loaded, then it will also react to resolve with the 'Unload' flag by unloading
	*   which may mean destructing an asset or destroying a spawned actor.
	*   Specific to Editor, some fragment types may spawn a 'preview' object/actor and have different behavior in runtime.
	*/
	CanBeLoaded = 1 << 0,

	/** Flag to indicate that this fragment type should be loaded by default. This flag is used to signal to the resolver of what this fragment type
	*   expects when being resolved.
	*/
	LoadedByDefault = 1 << 1,
};

ENUM_CLASS_FLAGS(EFragmentTypeFlags)


/**
 * A Universal Object Locator fragment type defines a specific mechanism for resolving an object that is referenced
 *   inside a Universal Object Locator (which can be considered a chain of fragments). Each fragment
 *   contains the attributes defining its payload, and how to interact that payload.
 *
 * In order to define a new fragment type, consumers must provide a UStruct 'payload' that defines the data required
 *   for the reference. The struct also defines how to interact with the payload via the following functions:
 *
 *   // Initialize the fragment to point at an object within a context
 *   FInitializeResult Initialize(const FInitializeParams& InParams) const;
 *   // Resolve the fragment to the resulting object, potentially loading or unloading it
 *   FResolveResult Resolve(const FResolveParams& Params) const;
 *
 *   // Convert this fragment to its string representation
 *   void ToString(FStringBuilderBase& OutStringBuilder) const;
 *   // Try and parse this fragment from its string representation
 *   FParseStringResult TryParseString(FStringView InString, const FParseStringParams& Params) const;
 *
 *   // Compute a priority for this fragment based on an object and context in order to decide which fragment type should be used if multiple can address the same object
 *   static uint32 ComputePriority(const UObject* Object, const UObject* Context) const;
 * 
 * Types must also be UStructs that are equality comparable, and GetTypeHashable.
 * These functions are bound as function pointers on construction, allowing them potentially to be inlined into the static dispatchers.
 */
struct FFragmentType
{
	FResolveResult ResolvePayload(const void* Payload, const FResolveParams& Params) const;
	FInitializeResult InitializePayload(void* Payload, const FInitializeParams& InParams) const;
	void ToString(const void* Payload, FStringBuilderBase& OutStringBuilder) const;
	FParseStringResult TryParseString(void* Payload, FStringView InString, const FParseStringParams& Params) const;

	uint32 ComputePriority(const UObject* Object, const UObject* Context) const;

	using FResolveCallback    = FResolveResult (*)(const void* /* Payload */, const FResolveParams& /* Params */);
	using FInitializeCallback = FInitializeResult (*)(void* /* Payload */, const FInitializeParams&);
	using FToString           = void (*)(const void* /* Payload */, FStringBuilderBase& /* OutStringBuilder */);
	using FTryParseString     = FParseStringResult (*)(void* /* Payload */, FStringView /* InString */, const FParseStringParams& /* Params */);

	using FPriorityCallback   = uint32 (*)(const UObject* /* Object */, const UObject* /* Context */);

#if UE_UNIVERSALOBJECTLOCATOR_DEBUG
	using FFragmentDebugInitializer = void (*)(void*);
#endif

	/** Bindings structure for function pointers that directly invoke behavior on fragment type payloads */
	struct
	{
		/** Function binding for initializing a payload with this fragment type */
		FInitializeCallback Initialize;
		/** Function binding for resolving a payload for this fragment type */
		FResolveCallback    Resolve;
		/** Function binding for serializing this payload to a string */
		FToString           ToString;
		/** Function binding for parsing this payload from a string */
		FTryParseString     TryParseString;
	} InstanceBindings;

	/** Bindings structure for function pointers that directly invoke behavior on fragment type payloads */
	struct
	{
		/** static function binding for computing the priority of this fragment type */
		FPriorityCallback   Priority;
#if UE_UNIVERSALOBJECTLOCATOR_DEBUG
		/** Function callback for initializing a vftable for the fragment for debugging purposes */
		FFragmentDebugInitializer FragmentDebugInitializer;
#endif
	} StaticBindings;

	/** Struct pointer to the payload type */
	TObjectPtr<const UScriptStruct> PayloadType;

	/** Descriptive display text for this type of fragment type */
	FText DisplayText;

	/** Unique identifier for this fragment type. This is serialized persistently to identify the fragment type, and exists as part of the fragment type's string representation */
	FName FragmentTypeID;

	/** Name of the primary editor type for this fragment type, defining how it appears on UI */
	FName PrimaryEditorType;

	/** Flags defining behavior of this fragment type */
	EFragmentTypeFlags Flags;

	UScriptStruct* GetStruct() const
	{
		return const_cast<UScriptStruct*>(PayloadType.Get());
	}
};

template<typename T>
struct TFragmentType : FFragmentType
{
};

} // namespace UE::UniversalObjectLocator