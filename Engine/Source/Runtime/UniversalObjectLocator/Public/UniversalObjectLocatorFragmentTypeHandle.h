// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/TypeHash.h"
#include "UniversalObjectLocatorFwd.h"

namespace UE::UniversalObjectLocator
{

/**
 * A handled to a globally registered Universal Object Locator fragment type.
 *
 * Registered through IUniversalObjectLocatorModule::RegisterFragmentType, fragment types define all the 
 *   attributes for a specific type of object resolution protocol that can be represented in a UOL.
 */
struct FFragmentTypeHandle
{
	FFragmentTypeHandle()
		: Handle(0xff)
	{
	}

	explicit FFragmentTypeHandle(uint8 InHandleOffset)
		: Handle(InHandleOffset)
	{
		check(InHandleOffset != 0xff);
	}

	explicit operator bool() const
	{
		return IsValid();
	}

	bool IsValid() const
	{
		return Handle != 0xff;
	}

	uint8 GetIndex() const
	{
		return Handle;
	}

	friend uint32 GetTypeHash(FFragmentTypeHandle In)
	{
		return GetTypeHash(In.Handle);
	}

	friend bool operator==(FFragmentTypeHandle A, FFragmentTypeHandle B)
	{
		return A.Handle == B.Handle;
	}

	friend bool operator!=(FFragmentTypeHandle A, FFragmentTypeHandle B)
	{
		return A.Handle != B.Handle;
	}

	friend bool operator<(FFragmentTypeHandle A, FFragmentTypeHandle B)
	{
		return A.Handle < B.Handle;
	}

	friend bool operator>(FFragmentTypeHandle A, FFragmentTypeHandle B)
	{
		return A.Handle > B.Handle;
	}

	UNIVERSALOBJECTLOCATOR_API FFragmentType* Resolve() const;

private:
	/** Global index into UE::UniversalObjectLocator::FRegistry::Get().FragmentTypes */
	uint8 Handle;
};


/**
 * Typed version of FFragmentTypeHandle that provides typed access to the underlying fragment type
 */
template<typename T>
struct TFragmentTypeHandle : FFragmentTypeHandle
{
	TFragmentTypeHandle()
	{
	}

	TFragmentType<T>* Resolve() const
	{
		return static_cast<TFragmentType<T>*>(FFragmentTypeHandle::Resolve());
	}

private:

	friend class IUniversalObjectLocatorModule;
	friend class FUniversalObjectLocatorModule;

	TFragmentTypeHandle(FFragmentTypeHandle InHandle)
		: FFragmentTypeHandle(InHandle)
	{
	}
};

} // namespace UE::UniversalObjectLocator