// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

struct FUniversalObjectLocator;
struct FUniversalObjectLocatorFragment;

template<typename> struct TUniversalObjectLocatorFragment;

enum class ELocatorResolveFlags : uint8;

namespace UE::UniversalObjectLocator
{

enum class ELocatorType : uint8;

struct FFragmentType;
template<typename> struct TFragmentType;

struct FFragmentTypeParameters;

struct FFragmentTypeHandle;
template<typename> struct TFragmentTypeHandle;

struct FParameterTypeHandle;
template<typename> struct TParameterTypeHandle;

struct FInitializeParams;
struct FInitializeResult;
struct FResolveParams;
struct FResolveResultFlags;
struct FResolveResult;
struct FParseStringParams;
struct FParseStringResult;

} // namespace UE::UniversalObjectLocator