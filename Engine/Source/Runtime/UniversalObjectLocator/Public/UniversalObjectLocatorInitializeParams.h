// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class UObject;

namespace UE::UniversalObjectLocator
{

/**
 * Fragment initialization parameter structure
 */
struct FInitializeParams
{
	/** The object that should be referenced by this locator (or, in the case of transient objects, an equivalent object). */
	const UObject* Object = nullptr;

	/** The context within which Object will be resolved within. Maybe null. */
	const UObject* Context = nullptr;

	/** Retrieve the context as a specific type. */
	template<typename T>
	const T* GetObjectAs() const
	{
		return Cast<T>(Object);
	}

	/** Retrieve the context as a specific type. */
	template<typename T>
	const T* GetContextAs() const
	{
		return Cast<const T>(Context);
	}
};

} // namespace UE::UniversalObjectLocator