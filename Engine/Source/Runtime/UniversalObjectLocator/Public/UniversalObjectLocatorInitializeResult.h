// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"

class UObject;

namespace UE::UniversalObjectLocator
{

enum class ELocatorType : uint8
{
	Absolute,
	Relative,
	Undefined,
};

/**
 * Fragment initialization result structure, specfying the type of locator that was created and an optional relative 'parent'
 */
struct FInitializeResult
{
	/** Optional object that this fragment should be resolved relative to */
	const UObject* RelativeToContext = nullptr;

	/** The type of the locator fragment */
	ELocatorType Type = ELocatorType::Undefined;

	static FInitializeResult Absolute()
	{
		FInitializeResult Result;
		Result.Type = ELocatorType::Absolute;
		return Result;
	}

	static FInitializeResult Relative(const UObject* InRelativeToContext)
	{
		check(InRelativeToContext);

		FInitializeResult Result;
		Result.RelativeToContext = InRelativeToContext;
		Result.Type = ELocatorType::Relative;
		return Result;
	}

	static FInitializeResult Failure()
	{
		FInitializeResult Result;
		Result.Type = ELocatorType::Undefined;
		return Result;
	}
};

} // namespace UE::UniversalObjectLocator