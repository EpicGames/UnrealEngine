// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/TypeHash.h"
#include "UniversalObjectLocatorFwd.h"

namespace UE::UniversalObjectLocator
{

/**
 * A handle to a globally registered Universal Object Locator fragment type.
 *
 * Registered through IUniversalObjectLocatorModule::RegisterParameterType, fragment types define all the 
 *   attributes for a specific type of object resolution protocol that can be represented in a UOL.
 */
struct FParameterTypeHandle
{
	FParameterTypeHandle()
		: Handle(0xff)
	{
	}

	explicit FParameterTypeHandle(uint8 InHandleOffset)
		: Handle(InHandleOffset)
	{
		check(InHandleOffset != 0xff);
	}

	explicit operator bool() const
	{
		return IsValid();
	}

	bool IsValid() const
	{
		return Handle != 0xff;
	}

	uint8 GetIndex() const
	{
		return Handle;
	}

	friend uint32 GetTypeHash(FParameterTypeHandle In)
	{
		return GetTypeHash(In.Handle);
	}

	friend bool operator==(FParameterTypeHandle A, FParameterTypeHandle B)
	{
		return A.Handle == B.Handle;
	}

	friend bool operator!=(FParameterTypeHandle A, FParameterTypeHandle B)
	{
		return A.Handle != B.Handle;
	}

	friend bool operator<(FParameterTypeHandle A, FParameterTypeHandle B)
	{
		return A.Handle < B.Handle;
	}

	friend bool operator>(FParameterTypeHandle A, FParameterTypeHandle B)
	{
		return A.Handle > B.Handle;
	}

	UNIVERSALOBJECTLOCATOR_API UScriptStruct* Resolve(UScriptStruct* Expected = nullptr) const;

private:
	/** Global index into UE::UniversalObjectLocator::FRegistry::Get().ParameterTypes */
	uint8 Handle;
};


/**
 * Typed version of FParameterTypeHandle that provides typed access to the underlying parameter type
 */
template<typename T>
struct TParameterTypeHandle : FParameterTypeHandle
{
	TParameterTypeHandle()
	{
	}

	UScriptStruct* Resolve() const
	{
		return FParameterTypeHandle::Resolve(T::StaticStruct());
	}

private:

	friend class IUniversalObjectLocatorModule;
	friend class FUniversalObjectLocatorModule;

	TParameterTypeHandle(FParameterTypeHandle InHandle)
		: FParameterTypeHandle(InHandle)
	{
	}
};

} // namespace UE::UniversalObjectLocator