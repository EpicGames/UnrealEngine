// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/UnrealMathUtility.h"
#include "UniversalObjectLocatorFwd.h"
#include "UniversalObjectLocatorParameterTypeHandle.h"

class UObject;

namespace UE::UniversalObjectLocator
{


/**
 * Header that describes an entry in the parameter buffer.
 */
struct FResolveParameterHeader
{
	// 2 Bytes
	uint16 ParameterOffset = 0;

	// 1 Byte
	uint8 Sizeof = 0;

	// 1 Byte
	uint8 Alignment = 0;

	/** Resolve the given raw pointer into a parameter pointer */
	void* Resolve(void* InMemory) const
	{
		return static_cast<uint8*>(InMemory) + ParameterOffset;
	}
};

/**
 * A buffer of parameter structs that can be optionally supplied to a UOL when resolving to supply additional context.
 *
 * Parameter types are registered through IUniversalObjectLocatorModule::RegisterParameterType and are identified by a unique index.
 * Parameter structs are stored contiguously in memory after an array of headers that specify the offset of each parameter type.
 *
 * Checking for the presence of a specific parameter type is very fast, amounting to a a simple bit operation.
 */
struct FResolveParameterBuffer
{
	/**
	 * Default constructor
	 */
	UNIVERSALOBJECTLOCATOR_API FResolveParameterBuffer();

	/**
	 * Destructor that calls each parameter's destructor, and frees the memory if necessary
	 */
	UNIVERSALOBJECTLOCATOR_API ~FResolveParameterBuffer();

	/**
	 * Non-copyable
	 */
	FResolveParameterBuffer(const FResolveParameterBuffer&) = delete;
	FResolveParameterBuffer& operator=(const FResolveParameterBuffer&) = delete;

	/**
	 * Non-moveable
	 */
	FResolveParameterBuffer(FResolveParameterBuffer&& RHS) = delete;
	FResolveParameterBuffer& operator=(FResolveParameterBuffer&& RHS) = delete;

public:

	/**
	 * Find a parameter of the specified type.
	 * Relies upon the struct type defining a static TParameterTypeHandle<T> called ParameterType that was previously registered through RegisterParameterType.
	 *
	 * @return A pointer to the parameter if present, nullptr otherwise
	 */
	template<typename T>
	const T* FindParameter() const;


	/**
	 * Find a parameter within this container that was added with the specified handle.
	 *
	 * @return A pointer to the parameter if present, nullptr otherwise
	 */
	template<typename T>
	const T* FindParameter(TParameterTypeHandle<T> ParameterTypeHandle) const;


	/**
	 * Add a new parameter to this container using its preregistered type handle.
	 * This function is accepts 0 or more additional construction arguments which are passed to the parameter's constructor.
	 *
	 * @note: The definition of this function is located in UniversalObjectLocatorResolveParameterBuffer.inl
	 *
	 * @return A pointer to the parameter if present, nullptr otherwise
	 */
	template<typename T, typename ...ArgTypes>
	T* AddParameter(TParameterTypeHandle<T> ParameterTypeHandle, ArgTypes&&... InArgs);

protected:

	/**
	 * Check whether the parameter for the specified parameter bit is present
	 */
	bool HasParameter(uint32 ParameterBit) const;

	/**
	 * Implementation function for finding a parameter ptr by its bit
	 */
	const void* FindParameterImpl(uint32 ParameterBit) const;


	/**
	 * Creates and stores a new Parameter object at the given bit.
	 * 
	 * @note: The definition of this function is located in UniversalObjectLocatorResolveParameterBuffer.inl
	 */
	template<typename ParameterType, typename ...ArgTypes>
	ParameterType* AddParameterImpl(uint32 ParameterBit, ArgTypes&&... InArgs);


	/**
	 * Retrieve the header at the specified index
	 */
	const FResolveParameterHeader& GetHeader(uint8 Index) const;


	/**
	 * Compute the index of the parameter that relates to the specified bit
	 */
	int32 GetParameterIndex(uint32 ParameterBit) const;


	/**
	 * Destroy this buffer and all its parameters
	 */
	void Destroy();

protected:

	/** Memory allocation of the layout [header_1|...|header_n|entry_0|...|entry_n] */
	uint8* Memory = nullptr;

	/** Bitmask of all the parameters contained within this container */
	uint32 AllParameters = 0u;

	/** Current capacity of Memory in bytes */
	uint16 Capacity = 0u;

	/** The number of parameters (and thus, headers) in this container */
	uint8 Num = 0u;

	/** Boolean that indicates whether we own the memory pointed to by Memory, or if it was supplied externally */
	bool bCanFreeMemory = false;

public:

	static constexpr int32 MaxNumParameters = sizeof(AllParameters) * 8;
};

/**
 * A FResolveParameterBuffer that has an inline memory capacity of the specified size
 */
template<int InlineByteSize>
struct alignas(8) TInlineResolveParameterBuffer : FResolveParameterBuffer
{
	TInlineResolveParameterBuffer()
	{
		Memory = InlineBuffer;
		Capacity = InlineByteSize;

		bCanFreeMemory = false;
	}

private:

	/** Inline memory buffer */
	uint8 InlineBuffer[InlineByteSize];
};


template<typename T>
const T* FResolveParameterBuffer::FindParameter() const
{
	return FindParameter(T::ParameterType);
}

template<typename T>
const T* FResolveParameterBuffer::FindParameter(TParameterTypeHandle<T> ParameterTypeHandle) const
{
	const uint8 ParameterIndex = ParameterTypeHandle.GetIndex();

	const uint32 ParameterBit = 1 << ParameterIndex;
	return static_cast<const T*>(FindParameterImpl(ParameterBit));
}

inline bool FResolveParameterBuffer::HasParameter(uint32 ParameterBit) const
{
	return (AllParameters & ParameterBit) != 0;
}

inline const void* FResolveParameterBuffer::FindParameterImpl(uint32 ParameterBit) const
{
	if (HasParameter(ParameterBit))
	{
		const int32 Index = GetParameterIndex(ParameterBit);
		check(Index >= 0 && Index < 255);
		return GetHeader(static_cast<uint8>(Index)).Resolve(Memory);
	}

	return nullptr;
}

inline const FResolveParameterHeader& FResolveParameterBuffer::GetHeader(uint8 Index) const
{
	check(Index < Num);
	return reinterpret_cast<FResolveParameterHeader*>(Memory)[Index];
}

inline int32 FResolveParameterBuffer::GetParameterIndex(uint32 ParameterBit) const
{
	if ( (AllParameters & ParameterBit) == 0)
	{
		return INDEX_NONE;
	}
	return FMath::CountBits(AllParameters & (ParameterBit-1u));
}

} // namespace UE::UniversalObjectLocator