// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UniversalObjectLocatorResolveParams.h"

namespace UE::UniversalObjectLocator
{


template<typename T, typename ...ArgTypes>
T* FResolveParameterBuffer::AddParameter(TParameterTypeHandle<T> ParameterTypeHandle, ArgTypes&&... InArgs)
{
	const UScriptStruct* ParameterStruct = ParameterTypeHandle.Resolve();
	if (!ensureMsgf(ParameterStruct, TEXT("Parameter struct is no longer registered!")))
	{
		return nullptr;
	}

	const uint8 ParameterIndex = ParameterTypeHandle.GetIndex();

	const uint32 ParameterBit = 1 << ParameterIndex;
	return static_cast<T*>(AddParameterImpl<T>(ParameterBit, Forward<ArgTypes>(InArgs)...));
}


template<typename ParameterType, typename ...ArgTypes>
ParameterType* FResolveParameterBuffer::AddParameterImpl(uint32 ParameterBit, ArgTypes&&... InArgs)
{
	checkf((AllParameters & ParameterBit) == 0, TEXT("Parameter already exists!"));

	// Add the enum entry
	AllParameters |= ParameterBit;

	// Find the index of the new Parameter by counting how many bits are set before it
	// For example, given ParameterBit=0b00010000 and AllParameters=0b00011011:
	//                       (ParameterBit-1) = 0b00001111
	//       AllParameters & (ParameterBit-1) = 0b00001011
	//                  CountBits(0b00001011) = 3
	const int32 NewParameterIndex = static_cast<int32>(FMath::CountBits(AllParameters & (ParameterBit-1u)));

	FResolveParameterHeader* ExistingHeaders = reinterpret_cast<FResolveParameterHeader*>(Memory);

	uint64 RequiredAlignment = FMath::Max(alignof(ParameterType), alignof(FResolveParameterHeader));

	const int32 ExistingNum = static_cast<int32>(Num);

	// Compute our required alignment for the allocation
	{
		for (int32 Index = 0; Index < ExistingNum; ++Index)
		{
			RequiredAlignment = FMath::Max(RequiredAlignment, (uint64)ExistingHeaders[Index].Alignment);
		}
	}

	uint64 RequiredSizeof = 0u;

	// Compute the required size of our allocation
	{
		// Allocate space for headers
		RequiredSizeof += (ExistingNum+1) * sizeof(FResolveParameterHeader);

		int32 Index = 0;

		// Count up the sizes and alignments of pre-existing parameters that exist before this new entry
		for (; Index < NewParameterIndex; ++Index)
		{
			RequiredSizeof = Align(RequiredSizeof, (uint64)ExistingHeaders[Index].Alignment);
			RequiredSizeof += ExistingHeaders[Index].Sizeof;
		}

		// Count up the size and alignment for the new parameter
		RequiredSizeof = Align(RequiredSizeof, alignof(ParameterType));
		RequiredSizeof += sizeof(ParameterType);
		++Index;

		// Now count up the sizes and alignments of pre-existing parameters that exist after this new entry
		for (; Index < ExistingNum+1; ++Index)
		{
			RequiredSizeof = Align(RequiredSizeof, (uint64)ExistingHeaders[Index-1].Alignment);
			RequiredSizeof += ExistingHeaders[Index-1].Sizeof;
		}
	}

	check( RequiredAlignment <= 0XFF );

	/// ----------------------------------------

	uint8* OldAllocation = Memory;

	const bool bShouldFreeMemory = bCanFreeMemory;

	// Make a new allocation if necessary
	const bool bNeedsReallocation = !IsAligned(Memory, RequiredAlignment) || RequiredSizeof > Capacity;
	if (bNeedsReallocation)
	{
		const uint64 RequiredCapacity = FMath::Max(RequiredSizeof, uint64(Capacity)*2);
		check(RequiredCapacity <= std::numeric_limits<uint16>::max());

		// Use the greater of the required size or double the current size to allow some additional capcity
		Capacity = static_cast<uint16>(FMath::Max(RequiredSizeof, uint64(Capacity)*2));
		Memory = reinterpret_cast<uint8*>(FMemory::Malloc(Capacity, RequiredAlignment));

		bCanFreeMemory = true;
	}

	// We now have an extra entry
	++Num;

	uint8* ParameterPtr = Memory + RequiredSizeof;

	auto RelocateParameter = [this, ExistingHeaders, OldAllocation, &ParameterPtr](int32 OldIndex)
	{
		// Go back
		ParameterPtr -= ExistingHeaders[OldIndex].Sizeof;
		ParameterPtr = AlignDown(ParameterPtr, ExistingHeaders[OldIndex].Alignment);

		void* OldParameter = ExistingHeaders[OldIndex].Resolve(OldAllocation);
		FMemory::Memmove(ParameterPtr, OldParameter, ExistingHeaders[OldIndex].Sizeof);

		// Overwrite the old header with its new position for error checking when we write the new header
		ExistingHeaders[OldIndex].ParameterOffset = static_cast<uint16>(ParameterPtr-Memory);
	};

	// Relocate old structs that proceed the new index
	for (int32 Index = static_cast<int32>(Num) - 1; Index > NewParameterIndex; --Index)
	{
		RelocateParameter(Index-1);
	}

	// Make the new entry
	{
		ParameterPtr -= sizeof(ParameterType);
		ParameterPtr = AlignDown(ParameterPtr, alignof(ParameterType));

		ParameterType* NewParameterPtr = reinterpret_cast<ParameterType*>(ParameterPtr);

		// Allocate the new type
		new (NewParameterPtr) ParameterType { Forward<ArgTypes>(InArgs)... };

		static_assert(alignof(ParameterType) < 0x7F, "Required alignment of parameter must fit in 7 bytes");
	}

	FResolveParameterHeader NewHeader = {
		static_cast<uint16>(ParameterPtr-Memory),
		sizeof(ParameterType),
		alignof(ParameterType)
	};

	// Relocate the entries that are before the new one
	for (int32 Index = NewParameterIndex-1; Index >= 0; --Index)
	{
		RelocateParameter(Index);
	}

	// Check that we have enough space before for the headers (this should always be the case)
	check(ParameterPtr >= Memory + sizeof(FResolveParameterHeader)*Num);

	FResolveParameterHeader* NewHeaders = reinterpret_cast<FResolveParameterHeader*>(Memory);

	for (int32 Index = static_cast<int32>(Num)-1; Index > NewParameterIndex; --Index)
	{
		new (&NewHeaders[Index]) FResolveParameterHeader(ExistingHeaders[Index-1]);
	};

	{
		new (&NewHeaders[NewParameterIndex]) FResolveParameterHeader(NewHeader);
	}

	if (Memory != OldAllocation)
	{
		for (int32 Index = NewParameterIndex-1; Index >= 0; --Index)
		{
			new (&NewHeaders[Index]) FResolveParameterHeader(ExistingHeaders[Index]);
		};
	}

	// Tidy up the old allocation. We do not call destructors here because we relocated everything.
	if (bNeedsReallocation && OldAllocation && bShouldFreeMemory)
	{
		FMemory::Free(OldAllocation);
	}

	return static_cast<ParameterType*>(NewHeaders[NewParameterIndex].Resolve(Memory));
}

} // namespace UE::UniversalObjectLocator