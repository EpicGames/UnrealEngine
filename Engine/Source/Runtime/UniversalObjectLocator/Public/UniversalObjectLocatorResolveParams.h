// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Async/Future.h"
#include "Misc/TVariant.h"
#include "UniversalObjectLocatorFwd.h"
#include "UniversalObjectLocatorResolveParameterBuffer.h"
#include "UniversalObjectLocatorResolveParams.generated.h"

class UObject;

UENUM(BlueprintType, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class ELocatorResolveFlags : uint8
{
	None,

	/** Flag to indicate whether the object should be loaded if it is not currently findable */
	Load = 1 << 0,

	/** Flag to indicate whether the object should be unloaded or destroyed. Mutually exclusive with bLoad. */
	Unload = 1 << 1,

	/**
	 * Indicates that the operation should be performed asynchronously if possible.
	 *   When not combined with WillWait, the caller will never block waiting for the result.
	 *   When combined with WillWait, the caller will block on this thread until the result is available,
	 *    so care needs to be taken avoid a deadlock if there are additional threading constraints on the load. */
	Async = 1 << 2,

	/** Indicates the calling code is going to block waiting for the result. */
	WillWait = 1 << 3,

	/** Combination of Async and WillWait. */
	AsyncWait = Async | WillWait,
};
ENUM_CLASS_FLAGS(ELocatorResolveFlags)

namespace UE::UniversalObjectLocator
{

/**
 * Parameters required to resolve a universal object locator
 */
struct FResolveParams
{
	FResolveParams()
		: Context(nullptr)
		, ParameterBuffer(nullptr)
		, Flags(ELocatorResolveFlags::None)
	{
	}

	FResolveParams(UObject* InContext)
		: Context(InContext)
		, ParameterBuffer(nullptr)
		, Flags(ELocatorResolveFlags::None)
	{
	}

	FResolveParams(UObject* InContext, ELocatorResolveFlags InFlags)
		: Context(InContext)
		, ParameterBuffer(nullptr)
		, Flags(InFlags)
	{
	}

	template<typename T>
	const T* FindParameter() const
	{
		return FindParameter(T::ParameterType);
	}

	template<typename T>
	const T* FindParameter(TParameterTypeHandle<T> ParameterType) const
	{
		return ParameterBuffer ? ParameterBuffer->FindParameter(ParameterType) : nullptr;
	}

	/** 
	 * Utility function to indicate that the object should be found, but not loaded or created if it is not currently
	 */
	static FResolveParams AsyncFind(UObject* InContext = nullptr)
	{
		return FResolveParams(InContext, ELocatorResolveFlags::Async);
	}

	/** 
	 * Utility function to indicate that the object should be loaded (or dynamically created) if it is not currently
	 */
	static FResolveParams AsyncLoad(UObject* InContext = nullptr)
	{
		return FResolveParams(InContext, ELocatorResolveFlags::Async | ELocatorResolveFlags::Load);
	}

	/** 
	 * Utility function to indicate that the object should be unloaded (or dynamically destroyed) if it is not currently
	 */
	static FResolveParams AsyncUnload(UObject* InContext)
	{
		return FResolveParams(InContext, ELocatorResolveFlags::Async | ELocatorResolveFlags::Unload);
	}

	/** 
	 * Utility function to indicate that the object should be found, but not loaded or created if it is not currently
	 */
	static FResolveParams SyncFind(UObject* InContext = nullptr)
	{
		return FResolveParams(InContext);
	}

	/** 
	 * Utility function to indicate that the object should be loaded (or dynamically created) if it is not currently
	 */
	static FResolveParams SyncLoad(UObject* InContext = nullptr)
	{
		return FResolveParams(InContext, ELocatorResolveFlags::Load);
	}

	/** 
	 * Utility function to indicate that the object should be unloaded (or dynamically destroyed) if it is not currently
	 */
	static FResolveParams SyncUnload(UObject* InContext)
	{
		return FResolveParams(InContext, ELocatorResolveFlags::Unload);
	}

	/** (Optional) Object to use as a context for resolution. Normally this is the object that owns the reference being resolved */
	UObject* Context;

	/** (Optional) Resolve buffer */
	FResolveParameterBuffer* ParameterBuffer;

	/** Flag structure */
	ELocatorResolveFlags Flags;
};

template<int InlineSize>
struct TResolveParamsWithBuffer : TInlineResolveParameterBuffer<InlineSize>, FResolveParams
{
	TResolveParamsWithBuffer()
	{
		ParameterBuffer = this;
	}

	TResolveParamsWithBuffer(UObject* InContext)
		: FResolveParams(InContext)
	{
		ParameterBuffer = this;
	}

	TResolveParamsWithBuffer(UObject* InContext, ELocatorResolveFlags InFlags)
		: FResolveParams(InContext, InFlags)
	{
		ParameterBuffer = this;
	}
};




/**
 * Flag structure returned from an attempt to resolve a Universal Object Locator
 */
struct FResolveResultFlags
{
	FResolveResultFlags()
		: bWasLoaded(0)
		, bWasLoadedIndirectly(0)
	{
	}

	/** Indicates the object was loaded as a result of this request */
	uint8 bWasLoaded : 1;
	/** Indicates that an outer object was loaded in order to fulfil this request, and the final object itself was loaded as a result */
	uint8 bWasLoadedIndirectly : 1;
};

struct FResolveResultData
{
	FResolveResultData()
		: Object(nullptr)
	{
	}

	FResolveResultData(UObject* InObject, FResolveResultFlags InFlags = FResolveResultFlags())
		: Object(InObject)
		, Flags(InFlags)
	{
	}

	/**
	 * The resulting object or nullptr if it could not be found or loaded per the request params.
	 * Should only be set if Params.Flags.bAsync is false.
	 */
	UObject* Object = nullptr;

	/** Flags relating to the operation */
	FResolveResultFlags Flags;
};


struct FResolveResult
{
	FResolveResult()
		: Value(TInPlaceType<FResolveResultData>())
	{
	}

	FResolveResult(FResolveResultData InValue)
		: Value(TInPlaceType<FResolveResultData>(), MoveTemp(InValue))
	{
	}

	FResolveResult(TFuture<FResolveResultData>&& InFuture)
		: Value(TInPlaceType<TFuture<FResolveResultData>>(), MoveTemp(InFuture))
	{
	}

	FResolveResult(const FResolveResult&) = delete;
	void operator=(const FResolveResult&) = delete;

	FResolveResult(FResolveResult&&) = default;
	FResolveResult& operator=(FResolveResult&&) = default;

public:

	/**
	 * Check whether this result is asynchronous
	 */
	bool IsAsync() const
	{
		return Value.IsType<TFuture<FResolveResultData>>();
	}

	/**
	 * Release the future from this type if it is set.
	 * WARNING: This can only be called once per instance. Invalidates this class.
	 * @return This result's future, or an empty one if the result has already been applied.
	 */
	TFuture<FResolveResultData> ReleaseFuture()
	{
		if (TFuture<FResolveResultData>* Future = Value.TryGet<TFuture<FResolveResultData>>())
		{
			return MoveTemp(*Future);
		}
		return TFuture<FResolveResultData>();
	}

	/**
	 * Check whether the result needs to be waited on.
	 * @return true if the result hasn't been set yet: SyncGet will block, and SyncGetNoWait will return null. If false, SyncGet will always return immediately, and SyncGetNoWait will never return null.
	 */
	bool NeedsWait() const
	{
		if (const TFuture<FResolveResultData>* Future = Value.TryGet<TFuture<FResolveResultData>>())
		{
			return !Future->IsReady();
		}
		return false;
	}

	/**
	 * Retrieve the result if it is currently available without waiting. Returns null if it is still being processed.
	 */
	const FResolveResultData* SyncGetNoWait()
	{
		const TFuture<FResolveResultData>* Future = Value.TryGet<TFuture<FResolveResultData>>();
		if (Future)
		{
			if (!Future->IsReady())
			{
				return nullptr;
			}

			FResolveResultData Data = Future->Get();
			Value.Emplace<FResolveResultData>(Data);
		}

		return &Value.Get<FResolveResultData>();
	}

	/**
	 * Retrieve the result, blocking this thread if necessary.
	 */
	FResolveResultData SyncGet() const
	{
		const TFuture<FResolveResultData>* Future = Value.TryGet<TFuture<FResolveResultData>>();
		if (Future)
		{
			return Future->Get();
		}
		else
		{
			return Value.Get<FResolveResultData>();
		}
	}

	/**
	 * Retrieve this result asynchronously.
	 * WARNING: This can only be called once per instance.
	 */
	void AsyncGet(TUniqueFunction<void(const FResolveResultData&)>&& OnComplete)
	{
		TFuture<FResolveResultData>* Future = Value.TryGet<TFuture<FResolveResultData>>();
		if (Future)
		{
			if (Future->IsReady())
			{
				OnComplete(Future->Get());
			}
			else
			{
				Future->Then(
					[OnComplete = MoveTemp(OnComplete)](TFuture<FResolveResultData>&& InValue)
					{
						OnComplete(InValue.Get());
					}
				);
			}
		}
		else
		{
			OnComplete(MoveTemp(Value.Get<FResolveResultData>()));
		}
	}

private:

	TVariant<
		FResolveResultData,
		TFuture<FResolveResultData>
	> Value;
};

} // namespace UE::UniversalObjectLocator