// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringView.h"
#include "Misc/EnumClassFlags.h"
#include "Internationalization/Text.h"

namespace UE::UniversalObjectLocator
{

enum class EParseStringFlags : uint8
{
	None = 0,

	/** Whether the user desires error messages to be emited */
	ErrorMessaging = 1 << 0,
};
ENUM_CLASS_FLAGS(EParseStringFlags)

/**
 * String parameter structure specifying additional information required for a string to be parsed
 */
struct FParseStringParams
{
	/** Parse flags */
	EParseStringFlags Flags = EParseStringFlags::None;

	bool NeedsErrorMessaging() const
	{
		return EnumHasAnyFlags(Flags, EParseStringFlags::ErrorMessaging);
	}
};

#define UE_UOL_PARSE_ERROR(InParams, ...) InParams.NeedsErrorMessaging() ? (__VA_ARGS__) : FText()

/**
 * String parse result structure, specfying success/failure, number of characters that were parsed, and
 *   any error information.
 */
struct FParseStringResult
{
	/** Optional error message, only populated when FParseStringParams::bWantsErrorMessaging is true */
	FText ErrorMessage;

	/** The number of characters that were parsed.
	 *   On Success this indicates the last character that resulted in the successful parse,
	 *   on failure, this may be 0, or the point at which parsing finished */
	int32 NumCharsParsed = 0;

	/** Whether parsing was successful or not */
	bool bSuccess = false;

	explicit operator bool() const
	{
		return bSuccess;
	}

	FParseStringResult& Success(int32 AdditionalNumCharsParsed = 0)
	{
		NumCharsParsed += AdditionalNumCharsParsed;
		bSuccess = true;
		return *this;
	}

	FParseStringResult& Failure(const FText& InFailureText)
	{
		if (!InFailureText.IsEmpty())
		{
			// Only overwrite the error message if it's supplied
			ErrorMessage = InFailureText;
		}
		bSuccess = false;
		return *this;
	}

	UNIVERSALOBJECTLOCATOR_API FStringView Progress(FStringView CurrentString, int32 NumToProgress);
};

} // namespace UE::UniversalObjectLocator