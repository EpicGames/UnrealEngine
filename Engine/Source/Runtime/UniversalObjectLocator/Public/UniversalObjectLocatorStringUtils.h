// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringFwd.h"

namespace UE::UniversalObjectLocator
{

	UNIVERSALOBJECTLOCATOR_API bool ParseUnsignedInteger(FStringView InString, uint32& OutResult, int32* OutNumCharsParsed = nullptr);

} // namespace UE::UniversalObjectLocator