// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UniversalObjectLocator : ModuleRules
{
	public UniversalObjectLocator(ReadOnlyTargetRules Target) : base(Target)
	{
        CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

        PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"HTTP",
			}
		);
	}
}
