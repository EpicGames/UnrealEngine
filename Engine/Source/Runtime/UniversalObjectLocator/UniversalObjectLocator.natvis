<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <Intrinsic Name="GetDebugState" SideEffect="false" Expression="GDebuggingState->Ptrs[(strstr(GDebuggingState->GuidString, GuidString) - GDebuggingState->GuidString)/32]">
    <Parameter Name="GuidString" Type="const char*"/>
  </Intrinsic>

  <Type Name="UE::UniversalObjectLocator::TFragmentPayload&lt;*&gt;">
    <DisplayString>{ ($T1*)Ptr,na }</DisplayString>
    <Expand>
      <ExpandedItem>($T1*)Ptr,nr</ExpandedItem>
    </Expand>
  </Type>
	
  <Type Name="FUniversalObjectLocatorFragment">
	<!-- Can't use GetDebugState here because we call GetType from the FUniversalObjectLocator visualizer, and the natvis evaluator optimizes it out and refuses to show it -->
    <Intrinsic Name="GetType" Expression="(((FDebuggableFragmentType::Type*)GDebuggingState->Ptrs[(GDebuggingState->GuidString - strstr(GDebuggingState->GuidString, &quot;49ceb527db044325a786a9b4470158fc&quot;))/32]) + FragmentType.Handle)"></Intrinsic>
    <Intrinsic Name="GetInlinePayload" Expression="(FDebuggableFragment::Type*)Data"></Intrinsic>
    <Intrinsic Name="GetHeapPayload" Expression="(FDebuggableFragment::Type*)((*(const void**)Data))"></Intrinsic>
    <Intrinsic Name="GetPayload" Category="Data" Expression="bIsInline ? GetInlinePayload() : GetHeapPayload(),nr"></Intrinsic>

    <DisplayString Condition="bIsInitialized == 0 &amp;&amp; FragmentType.Handle == 0xff">Empty</DisplayString>
    <DisplayString Condition="bIsInitialized == 0 &amp;&amp; FragmentType.Handle != 0xff">CORRUPT</DisplayString>
    <DisplayString Condition="bIsInitialized == 1 &amp;&amp; FragmentType.Handle == 0xff">CORRUPT</DisplayString>
    <DisplayString Condition="bIsInitialized == 1 &amp;&amp; FragmentType.Handle != 0xff">{GetPayload(),na}, Type={ (((FDebuggableFragmentType::Type*)GDebuggingState->Ptrs[(GDebuggingState->GuidString - strstr(GDebuggingState->GuidString, &quot;49ceb527db044325a786a9b4470158fc&quot;))/32]) + FragmentType.Handle)->PayloadType->NamePrivate,sb }</DisplayString>

    <Expand HideRawView="true">
		<ExpandedItem>GetPayload(),nanr</ExpandedItem>
		<Synthetic Name="[Raw View]">
			<DisplayString></DisplayString>
			<Expand HideRawView="true">
				<Item Name="Payload">GetPayload(),!</Item>
				<Item Name="Type">(((FDebuggableFragmentType::Type*)GDebuggingState->Ptrs[(GDebuggingState->GuidString - strstr(GDebuggingState->GuidString, &quot;49ceb527db044325a786a9b4470158fc&quot;))/32]) + FragmentType.Handle),na</Item>
			</Expand>
		</Synthetic>
    </Expand>
  </Type>

  <Type Name="FUniversalObjectLocator">
    <DisplayString Condition="Fragments.ArrayNum == 0">Empty</DisplayString>
    <DisplayString Condition="Fragments.ArrayNum == 1">{Fragments[0].GetType(),na}: {Fragments[0].GetPayload(),na}</DisplayString>
    <DisplayString Condition="Fragments.ArrayNum == 2">{Fragments[0].GetType(),na}/{Fragments[1].GetType(),na}: {Fragments[0].GetPayload(),na} / {Fragments[1].GetPayload(),na}</DisplayString>
    <DisplayString Condition="Fragments.ArrayNum == 3">{Fragments[0].GetType(),na}/{Fragments[1].GetType(),na}/{Fragments[2].GetType(),na}: {Fragments[0].GetPayload(),na} / {Fragments[1].GetPayload(),na} / {Fragments[2].GetPayload(),na}</DisplayString>
    <DisplayString Condition="Fragments.ArrayNum  > 3">{Fragments[0].GetType(),na}/{Fragments[1].GetType(),na}/{Fragments[2].GetType(),na}/...: {Fragments[0].GetPayload(),na} / {Fragments[1].GetPayload(),na} / {Fragments[2].GetPayload(),na}</DisplayString>
    <Expand HideRawView="true">
		<ExpandedItem>Fragments,nr</ExpandedItem>
    </Expand>
  </Type>
	
  <Type Name="UE::UniversalObjectLocator::FFragmentType">
    <DisplayString>{ FragmentTypeID,sb }</DisplayString>
  </Type>
</AutoVisualizer>
